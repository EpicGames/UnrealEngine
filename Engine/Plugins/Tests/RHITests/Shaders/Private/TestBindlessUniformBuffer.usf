// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

// Uniform buffer declared with BindlessAccessible.
// When SUPPORTS_BINDLESS_UNIFORM_BUFFER is enabled, the generated header exposes a
// GetBindlessAccessibleUniformBufferParametersConstants(uint Index) getter instead of a bound cbuffer.
#include "/Engine/Generated/UniformBuffers/BindlessAccessibleUniformBufferParameters.ush"

uint BindlessUBIndex;
RWStructuredBuffer<float4> OutputUAV;

[numthreads(1, 1, 1)]
void ReadBindlessUniformBuffer(uint3 GroupID : SV_GroupID)
{
	// Load the UB manually from the bindless heap using the index passed from C++.
	BindlessAccessibleUniformBufferParameters = GetBindlessAccessibleUniformBufferParametersConstants(BindlessUBIndex);
	OutputUAV[0] = BindlessAccessibleUniformBufferParameters.Value;
}
