// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

// Declare heap arrays for each resource type accessed via descriptor ranges.
// Required on Vulkan; no-op on other platforms.
DESCRIPTOR_RANGE_HEAP_DECL_SRV(Texture2D<float4>)
DESCRIPTOR_RANGE_HEAP_DECL_SRV(Buffer<float4>)
DESCRIPTOR_RANGE_HEAP_DECL_UAV(RWStructuredBuffer<float4>)

FDescriptorRange DescriptorRange;

RWStructuredBuffer<float4> OutputUAV;

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void FillFromTextures(uint3 GroupID : SV_GroupID)
{
	uint ResourceIndex = GroupID.x;
	Texture2D<float4> Tex = GetDescriptorRangeSRV(Texture2D<float4>, DescriptorRange, ResourceIndex);
	OutputUAV[ResourceIndex] = Tex.Load(uint3(0, 0, 0)).xyzw;
}

StructuredBuffer<float4> InputSRV;

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void WriteToUAVs(uint3 GroupID : SV_GroupID)
{
	uint ResourceIndex = GroupID.x;
	float4 Value = InputSRV[ResourceIndex];
	RWStructuredBuffer<float4> UAV = GetDescriptorRangeUAV(RWStructuredBuffer<float4>, DescriptorRange, ResourceIndex);
	UAV[0] = Value;
}

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void ReadSRVsWriteUAV(uint3 GroupID : SV_GroupID)
{
	uint ResourceIndex = GroupID.x;
	RWStructuredBuffer<float4> UAV = GetDescriptorRangeUAV(RWStructuredBuffer<float4>, DescriptorRange, 0);
	Texture2D<float4> Tex = GetDescriptorRangeSRV(Texture2D<float4>, DescriptorRange, ResourceIndex + 1);
	UAV[ResourceIndex] = Tex.Load(uint3(0, 0, 0)).xyzw;
}

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void FillFromMixedResources(uint3 GroupID : SV_GroupID)
{
	uint PairIndex = GroupID.x;
	Texture2D<float4> Tex = GetDescriptorRangeSRV(Texture2D<float4>, DescriptorRange, PairIndex * 2);
	Buffer<float4> Buf = GetDescriptorRangeSRV(Buffer<float4>, DescriptorRange, PairIndex * 2 + 1);
	OutputUAV[PairIndex * 2] = Tex.Load(uint3(0, 0, 0)).xyzw;
	OutputUAV[PairIndex * 2 + 1] = Buf[0];
}
