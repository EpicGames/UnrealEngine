// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

// Basic shader that writes 1 to the output buffer entries based on which instances were drawn.
// Used to validate that instanced draws with base instance offset work correctly.

void TestDrawInstancedMainVS(
	in float4 VertexPosition : ATTRIBUTE0,
	in uint CustomInstanceID : ATTRIBUTE1,
	out uint OutInstanceID : Texcoord0,
	out float4 OutPosition : SV_Position
)
{
	OutPosition = VertexPosition;
	OutInstanceID = CustomInstanceID;
}

RWBuffer<uint>	OutDrawnInstances;
void TestDrawInstancedMainPS(
	in nointerpolation uint InstanceID : Texcoord0,
	out float4 OutColor	: SV_Target0
)
{
	OutDrawnInstances[InstanceID] = 1;
	OutColor = float4(1,1,1,1);
}
