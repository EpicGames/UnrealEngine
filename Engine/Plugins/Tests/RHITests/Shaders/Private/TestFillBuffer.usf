// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

struct FStructuredBufferElem
{
	uint	IntVal;
	float	FloatVal;
};

RWBuffer<uint>								TypedBufferUAV;
RWByteAddressBuffer							ByteAddressBufferUAV;
RWStructuredBuffer<FStructuredBufferElem>	StructuredBufferUAV;
uint										NumElements;

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void FillBuffers(in uint3 ThreadID : SV_DispatchThreadID)
{
	uint Index = ThreadID.x;
	if(Index < NumElements)
	{
		uint TypedVal = Index;
		uint ByteAddressVal = Index + 1024;
		uint StructuredIntVal = Index + 2048;
		uint StructuredFloatVal = (float)(Index + 4096);

		TypedBufferUAV[Index] = TypedVal;
		ByteAddressBufferUAV.Store(Index * 4, ByteAddressVal);
		StructuredBufferUAV[Index].IntVal = StructuredIntVal;
		StructuredBufferUAV[Index].FloatVal = StructuredFloatVal;
	}
}

RWTexture2D<uint> Texture2DUAV;
RWTexture3D<uint> Texture3DUAV;
RWTexture2DArray<uint> TextureArrayUAV;
uint3 TextureSize;

[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, 1)]
void FillTextures(in uint3 ThreadID : SV_DispatchThreadID)
{
	if(ThreadID.x < TextureSize.x && ThreadID.y < TextureSize.y)
	{
		uint Texture2DVal = ThreadID.y * TextureSize.x + ThreadID.x;
		Texture2DUAV[ThreadID.xy] = Texture2DVal;

		for(int Z = 0; Z < TextureSize.z; ++Z)
		{
			uint Texture3DVal = Texture2DVal + (Z + 10) * 1024;
			Texture3DUAV[uint3(ThreadID.xy, Z)] = Texture3DVal;
		}

		for(int Slice = 0; Slice < TextureSize.z; ++Slice)
		{
			uint TextureArrayVal = Texture2DVal + (Slice + 10 + TextureSize.z) * 1024;
			TextureArrayUAV[uint3(ThreadID.xy, Slice)] = TextureArrayVal;
		}
	}
}
