// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"


float4 TestFillTextureConstant;

void TestFillTextureVS(
	in uint VertexID		: SV_VertexID,
	out float4 OutPosition	: SV_Position
)
{
#if 0
	// Produces broken code on some platforms
	switch(VertexID)
	{
		default:
		case 0:	OutPosition = float4(-1, -1, 0, 1); break;
		case 1:	OutPosition = float4(-1, +3, 0, 1); break;
		case 2:	OutPosition = float4(+3, -1, 0, 1); break;
	}
#else
	if      (VertexID == 0) { OutPosition = float4(-1, -1, 0, 1); }
	else if (VertexID == 1) { OutPosition = float4(-1, +3, 0, 1); }
	else                    { OutPosition = float4(+3, -1, 0, 1); }
#endif
}

void TestFillTexturePS(
	out float4 OutColor	: SV_Target0
)
{
	OutColor = TestFillTextureConstant;
}

#if SRC_TYPE == 0
Texture2D     <float4> SrcResource;
#elif SRC_TYPE == 1
Texture2DArray<float4> SrcResource;
#else
Texture3D     <float4> SrcResource;
#endif

RWTexture2D<float4> RWAtlas2D;
uint4 MipBiasMipNumsViewport;

// "unfolds" a given texture (2D/2D array/3D with mips) into a single 2D texture. X axis contains slices, Y axis contains mips
float4 TestTextureToAtlasCommon(uint2 PixelPositionInt)
{
	uint2 TexelCoords = 0;

	uint MipIndex = firstbithigh(PixelPositionInt.y + (1U << MipBiasMipNumsViewport.x));
	uint PixelStartMipY = (1U << (MipIndex - MipBiasMipNumsViewport.x)) - 1;
	TexelCoords.y = PixelPositionInt.y - PixelStartMipY;

	uint MipWidth = 1U << MipIndex;
	uint SliceIndex = PixelPositionInt.x / MipWidth;
	TexelCoords.x = (PixelPositionInt.x % MipWidth);

	uint TextureMipIndex = (MipBiasMipNumsViewport.y > 1) ? MipBiasMipNumsViewport.y - 1 - MipIndex : 0;

	uint4 LoadCoords = uint4(TexelCoords.xy, SliceIndex, TextureMipIndex);
#if SRC_TYPE == 0
	float4 TextureResult = SrcResource.Load(LoadCoords.xyw);
#else
	float4 TextureResult = SrcResource.Load(LoadCoords.xyzw);
#endif

	return TextureResult;
}

[numthreads(8, 8, 1)]
void TestTextureToAtlasCS(uint3 Position : SV_DispatchThreadID)
{
	if (all(Position.xy < MipBiasMipNumsViewport.zw))
	{
		float4 Output = TestTextureToAtlasCommon(Position.xy);
		RWAtlas2D[Position.xy] = Output;
	}
}
