// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

// NOTE: using a structured buffer here as workaround for UE-212251
RWStructuredBuffer<uint>	RWPixelShaderOutput;
RWStructuredBuffer<uint>	RWVertexShaderOutput;

void TestGraphicsUAVWriteMainVS(
	in float4 VertexPosition : ATTRIBUTE0,
	in uint VertexID : SV_VertexID,
	in uint InstanceID : SV_InstanceID,
	out uint OutInstanceID : Texcoord0,
	out float4 OutPosition : SV_Position
)
{
	RWVertexShaderOutput[VertexID] = VertexID;
	OutPosition = VertexPosition;
	OutInstanceID = InstanceID;
}

void TestGraphicsUAVTrivialMainVS(
	in float4 VertexPosition : ATTRIBUTE0,
	in uint InstanceID : SV_InstanceID,
	out uint OutInstanceID : Texcoord0,
	out float4 OutPosition : SV_Position
)
{
	OutPosition = VertexPosition;
	OutInstanceID = InstanceID;
}

void TestGraphicsUAVWriteMainPS(
	in nointerpolation uint InstanceID : Texcoord0,
	out float4 OutColor	: SV_Target0
)
{
	RWPixelShaderOutput[InstanceID] = InstanceID;
	OutColor = float4(1,1,1,1);
}
