// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"
#include "/Engine/Public/RootConstants.ush"

// ---- Compute shader ----

RWStructuredBuffer<uint4> OutputUAV;

[numthreads(1, 1, 1)]
void WriteRootConstants(uint3 GroupID : SV_GroupID)
{
	OutputUAV[0] = GetAllRootConstants();
}

// ---- Graphics shaders ----

// Trivial fullscreen VS — emits a full-screen triangle from vertex ID.
void RootConstantsMainVS(
	in  uint  VertexID  : SV_VertexID,
	out float4 OutPos   : SV_Position)
{
	// Two triangles covering the screen from three vertices
	float2 UV = float2((VertexID << 1) & 2, VertexID & 2);
	OutPos = float4(UV * float2(2, -2) + float2(-1, 1), 0, 1);
}

// PS — writes the root constant values into a UAV so they can be read back.
RWStructuredBuffer<uint4> GraphicsOutputUAV;

void RootConstantsMainPS(out float4 OutColor : SV_Target0)
{
	GraphicsOutputUAV[0] = GetAllRootConstants();
	OutColor = float4(0, 0, 0, 0);
}
