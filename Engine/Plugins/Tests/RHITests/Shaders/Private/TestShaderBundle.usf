// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

// Compute shader bundle test
// Each dispatch in the bundle writes its RecordIndex into the output
// buffer at the slot indicated by the indirect args (GroupCountX).
// GroupCountX encodes the record's output slot so we can verify
// which records actually executed.

RWBuffer<uint> OutBuffer;

[numthreads(1, 1, 1)]
void BundleComputeMain(uint3 ThreadID : SV_DispatchThreadID)
{
	OutBuffer[BundleTest.RecordSlot] = BundleTest.RecordSlot + 1; // write slot+1 so we can distinguish from zero-init
}

// Mesh + Pixel shader bundle test
// The mesh shader emits a single full-screen triangle.
// The pixel shader writes RecordSlot+1 into the output UAV.

struct FMeshToPixel
{
	float4 Position : SV_Position;
	uint SlotValue  : SLOT_VALUE;
};

#if SHADER_BUNDLE_TEST_MS

MESH_SHADER_TRIANGLE_ATTRIBUTES(64)
void BundleMeshMain(
	uint GroupThreadID : SV_GroupThreadID,
	MESH_SHADER_VERTEX_EXPORT(FMeshToPixel, 3),
	MESH_SHADER_TRIANGLE_EXPORT(1))
{
	SetMeshOutputCounts(3, 1);

	if (GroupThreadID < 1)
	{
		MESH_SHADER_WRITE_TRIANGLE(0, uint3(0, 1, 2));
	}

	if (GroupThreadID < 3)
	{
		// Full-screen triangle
		FMeshToPixel Vert;
		Vert.SlotValue = BundleTest.RecordSlot + 1;

		float2 Positions[3] = { float2(-1.0, -1.0), float2(-1.0, 3.0), float2(3.0, -1.0) };
		Vert.Position = float4(Positions[GroupThreadID], 0.0, 1.0);
		MESH_SHADER_WRITE_VERTEX(GroupThreadID, Vert);
	}
}

#endif // SHADER_BUNDLE_TEST_MS

#if SHADER_BUNDLE_TEST_PS

void BundlePixelMain(
	in FMeshToPixel Input,
	out float4 OutColor : SV_Target0)
{
	OutBuffer[Input.SlotValue - 1] = Input.SlotValue;
	OutColor = float4(0, 0, 0, 0);
}

#endif // SHADER_BUNDLE_TEST_PS

// Vertex + Pixel shader bundle test

#if SHADER_BUNDLE_TEST_VS

struct FVertexToPixel
{
	float4 Position : SV_Position;
};

void BundleVertexMain(
	uint VertexID : SV_VertexID,
	out FVertexToPixel Output)
{
	// Full-screen triangle from VertexID
	float2 UV = float2((VertexID << 1) & 2, VertexID & 2);
	Output.Position = float4(UV * float2(2.0, -2.0) + float2(-1.0, 1.0), 0.0, 1.0);
}

#endif // SHADER_BUNDLE_TEST_VS

#if SHADER_BUNDLE_TEST_VSPS_PS

void BundleVSPSPixelMain(
	in float4 SvPosition : SV_Position,
	out float4 OutColor : SV_Target0)
{
	OutBuffer[BundleTest.RecordSlot] = BundleTest.RecordSlot + 1;
	OutColor = float4(0, 0, 0, 0);
}

#endif // SHADER_BUNDLE_TEST_VSPS_PS
