// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIAllocatorTests.h"

bool FRHIAllocatorTests::Test_LockBuffer16ByteAlignment(FRHICommandListImmediate& RHICmdList)
{
	bool bSucceeded = true;

	const uint32 InAlignment = 0;
	const uint32 ExpectedOutAlignment = 16;

	const TArray<EBufferUsageFlags> InUsageFlags =
	{
		EBufferUsageFlags::Dynamic,
		EBufferUsageFlags::Static,
		EBufferUsageFlags::Volatile
	};

	const TArray<const TCHAR*> UsageFlagNames =
	{
		TEXT("Dynamic"),
		TEXT("Static"),
		TEXT("Volatile")
	};

	const int32 NumTests = InUsageFlags.Num();
	for (int32 TestIdx = 0; TestIdx < NumTests; ++TestIdx)
	{
		EBufferUsageFlags Flags = InUsageFlags[TestIdx];
		const FRHIBufferCreateDesc TestBufferCreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("TestLockBuffer16ByteAlignment"), uint32(sizeof(FVector4f) * 5), InAlignment, Flags | EBufferUsageFlags::VertexBuffer)
			.SetInitialState(ERHIAccess::CopyDest);

		const FBufferRHIRef TestBuffer = RHICmdList.CreateBuffer(TestBufferCreateDesc);

		uintptr_t MappedAddr = (uintptr_t)RHICmdList.LockBuffer(TestBuffer, 0, sizeof(FVector4f) * 5, RLM_WriteOnly);
		bool bResult = (MappedAddr % ExpectedOutAlignment) == 0;

		if (!bResult)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Test_LockBuffer16ByteAlignment\" (%ls)", UsageFlagNames[TestIdx]);
		}
		else
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"Test_LockBuffer16ByteAlignment\" (%ls)", UsageFlagNames[TestIdx]);
		}

		RHICmdList.UnlockBuffer(TestBuffer);
		bSucceeded &= bResult;
	}
	return bSucceeded;
}