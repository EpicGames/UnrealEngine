// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIBindlessTests.h"

#include "CommonRenderResources.h"
#include "RenderGraphUtils.h"
#include "RHI.h"
#include "RHIDescriptorRange.h"
#include "RHIGPUReadback.h"
#include "RHIResourceUtils.h"
#include "RHITestsCommon.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterMacros.h"
#include "ShaderPlatformConfig.h"

inline bool IsDescriptorTypeMaskSupported(ERHIDescriptorTypeMask TypeMask)
{
	for (ERHIDescriptorTypeMask SupportedGroup : GRHIGlobals.DescriptorTypeGroups)
	{
		if (EnumHasAllFlags(SupportedGroup, TypeMask))
		{
			return true;
		}
	}

	return false;
}

bool RHIBindlessTests::Test_ResourceCollection(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.bSupportsBindless || !IsBindlessEnabledForAnyGraphics(FShaderPlatformConfig::GetBindlessConfiguration(GMaxRHIShaderPlatform)))
	{
		return true;
	}

	const uint32 BufferContents[] = { 1, 2, 3, 4 };
	const uint32 TextureContents[] = { 1 };

	const FRHIBufferCreateDesc BufferDesc =
		FRHIBufferCreateDesc::CreateByteAddress(TEXT("ResourceCollection_Buffer"), sizeof(BufferContents), sizeof(BufferContents[0]))
		.SetInitialState(ERHIAccess::SRVMask)
		.SetInitActionInitializer();
	TRHIBufferInitializer<uint32> BufferInitializer = RHICmdList.CreateBufferInitializer(BufferDesc);
	{
		BufferInitializer.WriteArray(MakeConstArrayView(BufferContents));
	}
	FBufferRHIRef TestBuffer = BufferInitializer.Finalize();

	FRHIViewDesc::FBufferSRV::FInitializer BufferSRVDesc = FRHIViewDesc::CreateBufferSRV().SetType(FRHIViewDesc::EBufferType::Raw);
	FShaderResourceViewRHIRef BufferSRV = RHICmdList.CreateShaderResourceView(TestBuffer, BufferSRVDesc);

	const FRHITextureCreateDesc TextureDesc =
		FRHITextureCreateDesc::Create2D(TEXT("ResourceCollection_Texture"), 1, 1, PF_R32_UINT)
		.SetInitialState(ERHIAccess::SRVMask)
		.SetInitActionInitializer();
	FRHITextureInitializer TextureInitializer = RHICmdList.CreateTextureInitializer(TextureDesc);
	{
		TextureInitializer.GetTexture2DSubresource(0).WriteData(&TextureContents[0], sizeof(TextureContents));
	}
	FTextureRHIRef TestTexture = TextureInitializer.Finalize();

	// Empty
	{
		FRHIResourceCollectionRef EmptyCollection = RHICmdList.CreateResourceCollection({});
	}

	// Normal Creation
	{
		TArray<FRHIResourceCollectionMember> Members;
		Members.Emplace(BufferSRV.GetReference());
		Members.Emplace(TestTexture.GetReference());

		FRHIResourceCollectionRef ResourceCollection = RHICmdList.CreateResourceCollection(Members);
	}

	// Updates
	{
		TArray<FRHIResourceCollectionMember> Members;
		Members.Emplace(BufferSRV.GetReference());
		Members.Emplace(TestTexture.GetReference());

		FRHIResourceCollectionRef ResourceCollection = RHICmdList.CreateResourceCollection(Members);

		RHICmdList.UpdateResourceCollection(ResourceCollection, 0, Members);
	}

	return true;
}

class FDescriptorRangeTestCS : public FGlobalShader
{
public:
	using FGlobalShader::FGlobalShader;

	static constexpr uint32 ThreadGroupSizeX = 1;
	static constexpr uint32 ThreadGroupSizeY = 1;

	static bool ShouldCompilePermutation(const FShaderPermutationParameters& Parameters)
	{
		return IsBindlessEnabledForAnyGraphics(FShaderPlatformConfig::GetBindlessConfiguration(Parameters.Platform))
			&& FDataDrivenShaderPlatformInfo::GetSupportsDescriptorRange(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), ThreadGroupSizeY);
		OutEnvironment.CompilerFlags.Add(CFLAG_SupportsMinimalBindless);
	}
};

static bool SupportsDescriptorRangeTests()
{
	return GRHIGlobals.bSupportsBindless
		&& GRHIGlobals.bSupportsDescriptorRange
		&& IsBindlessEnabledForAnyGraphics(FShaderPlatformConfig::GetBindlessConfiguration(GMaxRHIShaderPlatform));
}

// Helper: Create a 1x1 texture with a given color and return it along with its SRV.
static void CreateColorTextureAndSRV(FRHICommandListBase& RHICmdList, FColor Color, FTextureRHIRef& OutTexture, FShaderResourceViewRHIRef& OutSRV)
{
	const FRHITextureCreateDesc TextureDesc =
		FRHITextureCreateDesc::Create2D(TEXT("Test_DescriptorRange"), 1, 1, PF_R8G8B8A8)
		.SetInitialState(ERHIAccess::SRVCompute)
		.SetInitActionInitializer();

	uint32 PackedColor = Color.ToPackedABGR();
	FRHITextureInitializer Initializer = RHICmdList.CreateTextureInitializer(TextureDesc);
	Initializer.GetTexture2DSubresource(0).WriteData(&PackedColor, sizeof(PackedColor));
	OutTexture = Initializer.Finalize();

	const FRHIViewDesc::FTextureSRV::FInitializer SRVInit =
		FRHIViewDesc::CreateTextureSRV()
		.SetDimension(TextureDesc.Dimension)
		.SetFormat(PF_R8G8B8A8)
		.SetMipRange(0, 1);

	OutSRV = RHICmdList.CreateShaderResourceView(OutTexture, SRVInit);
}

// Helper: Create an array of 1x1 color textures with SRVs and descriptor resources.
static void CreateColorTextureSRVResources(
	FRHICommandListBase& RHICmdList,
	TConstArrayView<FColor> Colors,
	TArray<FTextureRHIRef>& OutTextures,
	TArray<FShaderResourceViewRHIRef>& OutSRVs,
	TArray<FRHIDescriptorResource>& OutResources)
{
	for (int32 Index = 0; Index < Colors.Num(); Index++)
	{
		FTextureRHIRef Texture;
		FShaderResourceViewRHIRef SRV;
		CreateColorTextureAndSRV(RHICmdList, Colors[Index], Texture, SRV);
		OutResources.Emplace(SRV.GetReference());
		OutTextures.Emplace(MoveTemp(Texture));
		OutSRVs.Emplace(MoveTemp(SRV));
	}
}

// Helper: Readback a structured buffer of float4s and compare against expected colors.
static bool ReadbackAndVerifyColors(
	FRHICommandListImmediate& RHICmdList,
	FRHIBuffer* OutputBuffer,
	uint32 OutputBufferSize,
	int32 NumElements,
	TConstArrayView<FLinearColor> ExpectedColors,
	const TCHAR* TestName)
{
	FRHIGPUBufferReadback OutputBufferReadback(TEXT("OutputUAVReadback"));
	OutputBufferReadback.EnqueueCopy(RHICmdList, OutputBuffer, OutputBufferSize);

	RHICmdList.SubmitAndBlockUntilGPUIdle();

	bool bExpectedResult = true;

	{
		const FLinearColor* OutputData = reinterpret_cast<const FLinearColor*>(OutputBufferReadback.Lock(OutputBufferSize));
		for (int32 Index = 0; Index < NumElements; Index++)
		{
			const FLinearColor& ExpectedColor = ExpectedColors[Index];
			const FLinearColor& ActualColor = OutputData[Index];

			if (ExpectedColor != ActualColor)
			{
				UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"%ls\"\n\nIndex: %d;"
					" Expected Data: (%.2f, %.2f, %.2f, %.2f); Actual Data: (%.2f, %.2f, %.2f, %.2f);",
					TestName, Index,
					ExpectedColor.R, ExpectedColor.G, ExpectedColor.B, ExpectedColor.A,
					ActualColor.R, ActualColor.G, ActualColor.B, ActualColor.A);

				bExpectedResult = false;
				break;
			}
		}
		OutputBufferReadback.Unlock();

		if (bExpectedResult)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"%ls\"", TestName);
		}
	}

	return bExpectedResult;
}

// Helper: Dispatch FillFromTextures, readback, and compare against expected colors.

class FDescriptorRangeFillFromTexturesCS : public FDescriptorRangeTestCS
{
	DECLARE_GLOBAL_SHADER(FDescriptorRangeFillFromTexturesCS);
	SHADER_USE_PARAMETER_STRUCT(FDescriptorRangeFillFromTexturesCS, FDescriptorRangeTestCS);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWBuffer<float4>, OutputUAV)
		SHADER_PARAMETER_DESCRIPTOR_RANGE(FDescriptorRange, DescriptorRange)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FDescriptorRangeFillFromTexturesCS, "/Plugin/RHITests/Private/TestDescriptorRange.usf", "FillFromTextures", SF_Compute);

static bool DispatchAndVerifyDescriptorRange(
	FRHICommandListImmediate& RHICmdList,
	FDescriptorRangeRHIRef& DescriptorRange,
	int32 NumResources,
	TConstArrayView<FLinearColor> ExpectedColors,
	const TCHAR* TestName)
{
	FRHIBufferCreateDesc OutputBufferDesc =
		FRHIBufferCreateDesc::CreateStructured<FLinearColor>(TEXT("OutputUAV"), NumResources)
		.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute);
	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(OutputBufferDesc);

	FUnorderedAccessViewRHIRef OutputUAV = RHICmdList.CreateUnorderedAccessView(
		OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(OutputBufferDesc.Stride).SetNumElements(NumResources)
	);

	{
		TShaderMapRef<FDescriptorRangeFillFromTexturesCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
		const uint32 NumThreadGroups = FMath::DivideAndRoundUp<uint32>(NumResources, FDescriptorRangeFillFromTexturesCS::ThreadGroupSizeX);

		FDescriptorRangeFillFromTexturesCS::FParameters Parameters;
		Parameters.OutputUAV = OutputUAV;
		Parameters.DescriptorRange = DescriptorRange;

		FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(NumThreadGroups, 1, 1));
		RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));
	}

	return ReadbackAndVerifyColors(RHICmdList, OutputBuffer, OutputBufferDesc.Size, NumResources, ExpectedColors, TestName);
}

// Test reading from a descriptor range of texture SRVs.
// Creates SRVs from 1x1 color textures, reads them via a descriptor range, and verifies the output matches.
bool RHIBindlessTests::Test_DescriptorRange_SRV(FRHICommandListImmediate& RHICmdList)
{
	if (!SupportsDescriptorRangeTests())
	{
		return true;
	}

	const FColor TextureColors[] = { FColor::White, FColor::Black, FColor::Red, FColor::Green, FColor::Blue, FColor::Yellow };
	constexpr int32 NumResources = UE_ARRAY_COUNT(TextureColors);

	TArray<FTextureRHIRef> Textures;
	TArray<FShaderResourceViewRHIRef> SRVs;
	TArray<FRHIDescriptorResource> Resources;

	CreateColorTextureSRVResources(RHICmdList, TextureColors, Textures, SRVs, Resources);

	FRHIDescriptorRangeDesc RangeDesc;
	RangeDesc.Count = NumResources;
	RangeDesc.TypeMask = ERHIDescriptorTypeMask::SRV;

	FDescriptorRangeRHIRef DescriptorRange = RHICmdList.CreateDescriptorRange(RangeDesc, Resources);
	if (!DescriptorRange)
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Error, "Failed to allocate RHI Descriptor Range.");
		return false;
	}

	TArray<FLinearColor> ExpectedColors;
	for (int32 Index = 0; Index < NumResources; Index++)
	{
		ExpectedColors.Emplace(FLinearColor(TextureColors[Index]));
	}

	return DispatchAndVerifyDescriptorRange(RHICmdList, DescriptorRange, NumResources, ExpectedColors, TEXT("Descriptor Range SRV"));
}


// Test updating a subset of slots in an existing descriptor range.
bool RHIBindlessTests::Test_DescriptorRange_UpdateWithOffset(FRHICommandListImmediate& RHICmdList)
{
	if (!SupportsDescriptorRangeTests())
	{
		return true;
	}

	const FColor InitialColor = FColor::White;
	const FColor UpdatedColors[] = { FColor::Red, FColor::Green, FColor::Blue };

	constexpr int32 NumSlots = 6;
	constexpr int32 UpdateStartIndex = 2;
	constexpr int32 NumUpdated = UE_ARRAY_COUNT(UpdatedColors);

	// Create initial textures — all white.
	TArray<FColor> InitialColors;
	InitialColors.SetNum(NumSlots);
	for (FColor& Color : InitialColors) { Color = InitialColor; }

	TArray<FTextureRHIRef> Textures;
	TArray<FShaderResourceViewRHIRef> SRVs;
	TArray<FRHIDescriptorResource> Resources;

	CreateColorTextureSRVResources(RHICmdList, InitialColors, Textures, SRVs, Resources);

	FRHIDescriptorRangeDesc RangeDesc;
	RangeDesc.Count = NumSlots;
	RangeDesc.TypeMask = ERHIDescriptorTypeMask::SRV;

	FDescriptorRangeRHIRef DescriptorRange = RHICmdList.CreateDescriptorRange(RangeDesc, Resources);
	if (!DescriptorRange)
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Error, "Failed to allocate RHI Descriptor Range.");
		return false;
	}

	// Update slots 2, 3, 4 with new colors.
	TArray<FTextureRHIRef> UpdatedTextures;
	TArray<FShaderResourceViewRHIRef> UpdatedSRVs;
	TArray<FRHIDescriptorResource> UpdatedResources;

	CreateColorTextureSRVResources(RHICmdList, UpdatedColors, UpdatedTextures, UpdatedSRVs, UpdatedResources);

	RHICmdList.UpdateDescriptorRange(DescriptorRange, UpdateStartIndex, UpdatedResources);

	// Expected: White, White, Red, Green, Blue, White
	TArray<FLinearColor> ExpectedColors;
	for (int32 Index = 0; Index < NumSlots; Index++)
	{
		if (Index >= UpdateStartIndex && Index < UpdateStartIndex + NumUpdated)
		{
			ExpectedColors.Emplace(FLinearColor(UpdatedColors[Index - UpdateStartIndex]));
		}
		else
		{
			ExpectedColors.Emplace(FLinearColor(InitialColor));
		}
	}

	return DispatchAndVerifyDescriptorRange(RHICmdList, DescriptorRange, NumSlots, ExpectedColors, TEXT("Descriptor Range Update With Offset"));
}

class FDescriptorRangeWriteToUAVsCS : public FDescriptorRangeTestCS
{
	DECLARE_GLOBAL_SHADER(FDescriptorRangeWriteToUAVsCS);
	SHADER_USE_PARAMETER_STRUCT(FDescriptorRangeWriteToUAVsCS, FDescriptorRangeTestCS);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_SRV(StructuredBuffer<float4>, InputSRV)
		SHADER_PARAMETER_DESCRIPTOR_RANGE(FDescriptorRange, DescriptorRange)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FDescriptorRangeWriteToUAVsCS, "/Plugin/RHITests/Private/TestDescriptorRange.usf", "WriteToUAVs", SF_Compute);

// Test writing to a descriptor range of UAVs.
bool RHIBindlessTests::Test_DescriptorRange_UAV(FRHICommandListImmediate& RHICmdList)
{
	if (!SupportsDescriptorRangeTests())
	{
		return true;
	}

	constexpr int32 NumResources = 4;

	const FLinearColor InputValues[NumResources] =
	{
		FLinearColor(1.0f, 0.0f, 0.0f, 1.0f),
		FLinearColor(0.0f, 1.0f, 0.0f, 1.0f),
		FLinearColor(0.0f, 0.0f, 1.0f, 1.0f),
		FLinearColor(1.0f, 1.0f, 0.0f, 1.0f),
	};

	// Create an input buffer with the values we want to write through the UAV range.
	FRHIBufferCreateDesc InputBufferDesc =
		FRHIBufferCreateDesc::CreateStructured<FLinearColor>(TEXT("InputSRV"), NumResources)
		.SetInitialState(ERHIAccess::SRVCompute)
		.SetInitActionInitializer();
	TRHIBufferInitializer<FLinearColor> InputInitializer = RHICmdList.CreateBufferInitializer(InputBufferDesc);
	{
		InputInitializer.WriteArray(MakeConstArrayView(InputValues));
	}
	FBufferRHIRef InputBuffer = InputInitializer.Finalize();

	FShaderResourceViewRHIRef InputSRV = RHICmdList.CreateShaderResourceView(
		InputBuffer,
		FRHIViewDesc::CreateBufferSRV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(sizeof(FLinearColor)).SetNumElements(NumResources)
	);

	// Create output UAV buffers — one per descriptor range slot.
	TArray<FBufferRHIRef> OutputBuffers;
	TArray<FUnorderedAccessViewRHIRef> OutputUAVs;
	TArray<FRHIDescriptorResource> Resources;

	for (int32 Index = 0; Index < NumResources; Index++)
	{
		FRHIBufferCreateDesc OutputBufferDesc =
			FRHIBufferCreateDesc::CreateStructured<FLinearColor>(TEXT("UAVRangeOutput"), 1)
			.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy)
			.SetInitialState(ERHIAccess::UAVCompute);
		FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(OutputBufferDesc);

		FUnorderedAccessViewRHIRef OutputUAV = RHICmdList.CreateUnorderedAccessView(
			OutputBuffer,
			FRHIViewDesc::CreateBufferUAV()
				.SetType(FRHIViewDesc::EBufferType::Structured)
				.SetStride(sizeof(FLinearColor)).SetNumElements(1)
		);

		Resources.Emplace(OutputUAV.GetReference());
		OutputBuffers.Emplace(MoveTemp(OutputBuffer));
		OutputUAVs.Emplace(MoveTemp(OutputUAV));
	}

	FRHIDescriptorRangeDesc RangeDesc;
	RangeDesc.Count = NumResources;
	RangeDesc.TypeMask = ERHIDescriptorTypeMask::UAV;

	FDescriptorRangeRHIRef DescriptorRange = RHICmdList.CreateDescriptorRange(RangeDesc, Resources);
	if (!DescriptorRange)
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Error, "Failed to allocate RHI Descriptor Range for UAVs.");
		return false;
	}

	// Dispatch the WriteToUAVs shader.
	{
		TShaderMapRef<FDescriptorRangeWriteToUAVsCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FDescriptorRangeWriteToUAVsCS::FParameters Parameters;
		Parameters.InputSRV = InputSRV;
		Parameters.DescriptorRange = DescriptorRange;

		FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(NumResources, 1, 1));

		for (const FUnorderedAccessViewRHIRef& UAV : OutputUAVs)
		{
			RHICmdList.Transition(FRHITransitionInfo(UAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));
		}
	}

	RHICmdList.SubmitAndBlockUntilGPUIdle();

	// Readback each output buffer and verify.
	bool bExpectedResult = true;

	for (int32 Index = 0; Index < NumResources; Index++)
	{
		FRHIGPUBufferReadback Readback(TEXT("UAVRangeReadback"));
		Readback.EnqueueCopy(RHICmdList, OutputBuffers[Index], sizeof(FLinearColor));
		RHICmdList.SubmitAndBlockUntilGPUIdle();

		const FLinearColor* OutputData = reinterpret_cast<const FLinearColor*>(Readback.Lock(sizeof(FLinearColor)));
		const FLinearColor& ExpectedColor = InputValues[Index];
		const FLinearColor& ActualColor = *OutputData;

		if (ExpectedColor != ActualColor)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Descriptor Range UAV\"\n\nIndex: %d;"
				" Expected Data: (%.2f, %.2f, %.2f, %.2f); Actual Data: (%.2f, %.2f, %.2f, %.2f);",
				Index,
				ExpectedColor.R, ExpectedColor.G, ExpectedColor.B, ExpectedColor.A,
				ActualColor.R, ActualColor.G, ActualColor.B, ActualColor.A);

			bExpectedResult = false;
		}
		Readback.Unlock();

		if (!bExpectedResult)
		{
			break;
		}
	}

	if (bExpectedResult)
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"Descriptor Range UAV\"");
	}

	return bExpectedResult;
}

class FDescriptorRangeReadSRVsWriteUAVCS : public FDescriptorRangeTestCS
{
	DECLARE_GLOBAL_SHADER(FDescriptorRangeReadSRVsWriteUAVCS);
	SHADER_USE_PARAMETER_STRUCT(FDescriptorRangeReadSRVsWriteUAVCS, FDescriptorRangeTestCS);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_DESCRIPTOR_RANGE(FDescriptorRange, DescriptorRange)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FDescriptorRangeReadSRVsWriteUAVCS, "/Plugin/RHITests/Private/TestDescriptorRange.usf", "ReadSRVsWriteUAV", SF_Compute);

// Test a descriptor range containing both SRV and UAV resources.
// Places a single UAV at index 0 followed by texture SRVs. The shader reads each SRV and writes to the UAV, verifying both view types work in the same range.
bool RHIBindlessTests::Test_DescriptorRange_SRV_And_UAV(FRHICommandListImmediate& RHICmdList)
{
	if (!SupportsDescriptorRangeTests())
	{
		return true;
	}

	if (!IsDescriptorTypeMaskSupported(ERHIDescriptorTypeMask::SRV | ERHIDescriptorTypeMask::UAV))
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Display, "SRV+UAV Descriptor Range is not supported on this platform. Skipping test.");
		return true;
	}

	// Test a descriptor range containing a single UAV followed by SRVs: [UAV, SRV0, SRV1, SRV2, SRV3]
	// The shader reads from each SRV and writes the result into the UAV at the corresponding index.
	const FColor TextureColors[] = { FColor::Red, FColor::Green, FColor::Blue, FColor::Yellow };
	constexpr int32 NumTextures = UE_ARRAY_COUNT(TextureColors);
	constexpr int32 NumResources = 1 + NumTextures; // 1 UAV + N SRVs

	// Create the output UAV buffer — holds one float4 per texture.
	FRHIBufferCreateDesc OutputBufferDesc =
		FRHIBufferCreateDesc::CreateStructured<FLinearColor>(TEXT("SRVAndUAVRangeOutput"), NumTextures)
		.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute);
	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(OutputBufferDesc);

	FUnorderedAccessViewRHIRef OutputUAV = RHICmdList.CreateUnorderedAccessView(
		OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(sizeof(FLinearColor)).SetNumElements(NumTextures)
	);

	// Build the resource list: UAV at index 0, then SRVs.
	TArray<FTextureRHIRef> Textures;
	TArray<FShaderResourceViewRHIRef> SRVs;
	TArray<FRHIDescriptorResource> Resources;

	Resources.Emplace(OutputUAV.GetReference());

	CreateColorTextureSRVResources(RHICmdList, TextureColors, Textures, SRVs, Resources);

	FRHIDescriptorRangeDesc RangeDesc;
	RangeDesc.Count = NumResources;
	RangeDesc.TypeMask = ERHIDescriptorTypeMask::SRV | ERHIDescriptorTypeMask::UAV;

	FDescriptorRangeRHIRef DescriptorRange = RHICmdList.CreateDescriptorRange(RangeDesc, Resources);
	if (!DescriptorRange)
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Display, "Unable to allocate RHI Descriptor Range for SRV+UAV.");
		return false;
	}

	// Dispatch the ReadSRVsWriteUAV shader.
	{
		TShaderMapRef<FDescriptorRangeReadSRVsWriteUAVCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FDescriptorRangeReadSRVsWriteUAVCS::FParameters Parameters;
		Parameters.DescriptorRange = DescriptorRange;

		FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(NumTextures, 1, 1));

		RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));
	}

	TArray<FLinearColor> ExpectedColors;
	for (int32 Index = 0; Index < NumTextures; Index++)
	{
		ExpectedColors.Emplace(FLinearColor(TextureColors[Index]));
	}

	return ReadbackAndVerifyColors(RHICmdList, OutputBuffer, OutputBufferDesc.Size, NumTextures, ExpectedColors, TEXT("Descriptor Range SRV And UAV"));
}

class FDescriptorRangeFillFromMixedResourcesCS : public FDescriptorRangeTestCS
{
	DECLARE_GLOBAL_SHADER(FDescriptorRangeFillFromMixedResourcesCS);
	SHADER_USE_PARAMETER_STRUCT(FDescriptorRangeFillFromMixedResourcesCS, FDescriptorRangeTestCS);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWBuffer<float4>, OutputUAV)
		SHADER_PARAMETER_DESCRIPTOR_RANGE(FDescriptorRange, DescriptorRange)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FDescriptorRangeFillFromMixedResourcesCS, "/Plugin/RHITests/Private/TestDescriptorRange.usf", "FillFromMixedResources", SF_Compute);

// Test a descriptor range containing a mix of texture SRVs and buffer SRVs.
bool RHIBindlessTests::Test_DescriptorRange_MixedResourceTypes(FRHICommandListImmediate& RHICmdList)
{
	if (!SupportsDescriptorRangeTests())
	{
		return true;
	}

	// Mix of textures and buffers in the same range, interleaved as pairs: [Tex0, Buf0, Tex1, Buf1, ...]
	// The shader reads from each texture and buffer and writes both to the output.
	const FColor Colors[] = { FColor::Red, FColor::Green, FColor::Blue, FColor::Yellow };
	constexpr int32 NumPairs = UE_ARRAY_COUNT(Colors);
	constexpr int32 NumResources = NumPairs * 2; // Texture + Buffer per pair

	TArray<FTextureRHIRef> Textures;
	TArray<FShaderResourceViewRHIRef> TextureSRVs;
	TArray<FBufferRHIRef> Buffers;
	TArray<FShaderResourceViewRHIRef> BufferSRVs;
	TArray<FRHIDescriptorResource> Resources;

	for (int32 Index = 0; Index < NumPairs; Index++)
	{
		// Even slot: 1x1 texture SRV
		FTextureRHIRef Texture;
		FShaderResourceViewRHIRef TextureSRV;
		CreateColorTextureAndSRV(RHICmdList, Colors[Index], Texture, TextureSRV);

		// Odd slot: buffer SRV containing the same color
		FLinearColor BufferColor(Colors[Index]);
		FRHIBufferCreateDesc BufferDesc =
			FRHIBufferCreateDesc::Create(TEXT("MixedRangeBuffer"), sizeof(FLinearColor), sizeof(FLinearColor), EBufferUsageFlags::ShaderResource)
			.SetInitialState(ERHIAccess::SRVCompute)
			.SetInitActionInitializer();
		TRHIBufferInitializer<FLinearColor> BufferInitializer = RHICmdList.CreateBufferInitializer(BufferDesc);
		{
			BufferInitializer.WriteData(&BufferColor, sizeof(BufferColor));
		}
		FBufferRHIRef Buffer = BufferInitializer.Finalize();

		FShaderResourceViewRHIRef BufferSRV = RHICmdList.CreateShaderResourceView(
			Buffer,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_A32B32G32R32F)
		);

		Resources.Emplace(TextureSRV.GetReference());
		Resources.Emplace(BufferSRV.GetReference());

		Textures.Emplace(MoveTemp(Texture));
		TextureSRVs.Emplace(MoveTemp(TextureSRV));
		Buffers.Emplace(MoveTemp(Buffer));
		BufferSRVs.Emplace(MoveTemp(BufferSRV));
	}

	FRHIDescriptorRangeDesc RangeDesc;
	RangeDesc.Count = NumResources;
	RangeDesc.TypeMask = ERHIDescriptorTypeMask::SRV;

	FDescriptorRangeRHIRef DescriptorRange = RHICmdList.CreateDescriptorRange(RangeDesc, Resources);
	if (!DescriptorRange)
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Error, "Failed to allocate RHI Descriptor Range.");
		return false;
	}

	// Create output buffer and UAV.
	FRHIBufferCreateDesc OutputBufferDesc =
		FRHIBufferCreateDesc::CreateStructured<FLinearColor>(TEXT("OutputUAV"), NumResources)
		.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute);
	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(OutputBufferDesc);

	FUnorderedAccessViewRHIRef OutputUAV = RHICmdList.CreateUnorderedAccessView(
		OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(OutputBufferDesc.Stride).SetNumElements(NumResources)
	);

	{
		TShaderMapRef<FDescriptorRangeFillFromMixedResourcesCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FDescriptorRangeFillFromMixedResourcesCS::FParameters Parameters;
		Parameters.OutputUAV = OutputUAV;
		Parameters.DescriptorRange = DescriptorRange;

		FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(NumPairs, 1, 1));
		RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));
	}

	// Expected: each color appears twice (once from texture, once from buffer).
	TArray<FLinearColor> ExpectedColors;
	for (int32 Index = 0; Index < NumPairs; Index++)
	{
		FLinearColor Color(Colors[Index]);
		ExpectedColors.Emplace(Color); // from texture
		ExpectedColors.Emplace(Color); // from buffer
	}

	return ReadbackAndVerifyColors(RHICmdList, OutputBuffer, OutputBufferDesc.Size, NumResources, ExpectedColors, TEXT("Descriptor Range Mixed Resource Types"));
}

// Uniform buffer struct used to test BindlessAccessible uniform buffers.
BEGIN_UNIFORM_BUFFER_STRUCT(FBindlessAccessibleUniformBufferParameters, )
	SHADER_PARAMETER(FVector4f, Value)
END_UNIFORM_BUFFER_STRUCT()
IMPLEMENT_UNIFORM_BUFFER_STRUCT_EX(FBindlessAccessibleUniformBufferParameters, "BindlessAccessibleUniformBufferParameters", FShaderParametersMetadata::EUsageFlags::BindlessAccessible);

static bool GEnableBindlessUniformBufferTest = false;

class FBindlessUniformBufferTestCS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FBindlessUniformBufferTestCS);
	SHADER_USE_PARAMETER_STRUCT(FBindlessUniformBufferTestCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FBindlessAccessibleUniformBufferParameters, BindlessUB)
		SHADER_PARAMETER(uint32, BindlessUBIndex)
		SHADER_PARAMETER_UAV(RWStructuredBuffer<float4>, OutputUAV)
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FShaderPermutationParameters& Parameters)
	{
		return GEnableBindlessUniformBufferTest && IsBindlessEnabledForAnyGraphics(FShaderPlatformConfig::GetBindlessConfiguration(Parameters.Platform));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_SupportsMinimalBindless);
		OutEnvironment.SetDefine(TEXT("SUPPORTS_BINDLESS_UNIFORM_BUFFER"), 1);
	}
};
IMPLEMENT_GLOBAL_SHADER(FBindlessUniformBufferTestCS, "/Plugin/RHITests/Private/TestBindlessUniformBuffer.usf", "ReadBindlessUniformBuffer", SF_Compute);

// Test that a uniform buffer declared with BindlessAccessible gets a valid bindless handle,
// and that its contents are correctly accessible when bound to a shader.
bool RHIBindlessTests::Test_BindlessAccessibleUniformBuffer(FRHICommandListImmediate& RHICmdList)
{
	if (!GEnableBindlessUniformBufferTest || !GRHIGlobals.bSupportsBindless || !IsBindlessEnabledForAnyGraphics(FShaderPlatformConfig::GetBindlessConfiguration(GMaxRHIShaderPlatform)))
	{
		return true;
	}

	const FVector4f TestValue(1.0f, 2.0f, 3.0f, 4.0f);

	FBindlessAccessibleUniformBufferParameters UBContents;
	UBContents.Value = TestValue;

	TUniformBufferRef<FBindlessAccessibleUniformBufferParameters> UniformBuffer =
		CreateUniformBufferImmediate(UBContents, EUniformBufferUsage::UniformBuffer_SingleFrame);

	if (!UniformBuffer)
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Bindless Accessible Uniform Buffer\": Failed to create uniform buffer.");
		return false;
	}

	// Verify the bindless handle was allocated.
	const FRHIDescriptorHandle BindlessHandle = UniformBuffer->GetBindlessHandle();
	if (!BindlessHandle.IsValid())
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Bindless Accessible Uniform Buffer\": Uniform buffer does not have a valid bindless handle.");
		return false;
	}

	// Dispatch a shader that reads the UB value and writes it to an output buffer.
	const FRHIBufferCreateDesc OutputBufferDesc =
		FRHIBufferCreateDesc::CreateStructured<FLinearColor>(TEXT("BindlessUBTestOutput"), 1)
		.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute);
	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(OutputBufferDesc);

	FUnorderedAccessViewRHIRef OutputUAV = RHICmdList.CreateUnorderedAccessView(
		OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(sizeof(FLinearColor)).SetNumElements(1)
	);

	{
		TShaderMapRef<FBindlessUniformBufferTestCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		FBindlessUniformBufferTestCS::FParameters Parameters;
		Parameters.BindlessUB = UniformBuffer;
		Parameters.BindlessUBIndex = UniformBuffer->GetBindlessHandle().GetIndex();
		Parameters.OutputUAV = OutputUAV;

		FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(1, 1, 1));

		RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));
	}

	const TArray<FLinearColor> ExpectedColors = { FLinearColor(TestValue.X, TestValue.Y, TestValue.Z, TestValue.W) };
	return ReadbackAndVerifyColors(RHICmdList, OutputBuffer, OutputBufferDesc.Size, 1, ExpectedColors, TEXT("Bindless Accessible Uniform Buffer"));
}
