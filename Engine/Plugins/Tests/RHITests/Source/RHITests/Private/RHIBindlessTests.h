// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FRHICommandListImmediate;

namespace RHIBindlessTests
{
	bool Test_ResourceCollection(FRHICommandListImmediate& RHICmdList);
	bool Test_DescriptorRange_SRV(FRHICommandListImmediate& RHICmdList);
	bool Test_DescriptorRange_UpdateWithOffset(FRHICommandListImmediate& RHICmdList);
	bool Test_DescriptorRange_UAV(FRHICommandListImmediate& RHICmdList);
	bool Test_DescriptorRange_SRV_And_UAV(FRHICommandListImmediate& RHICmdList);
	bool Test_DescriptorRange_MixedResourceTypes(FRHICommandListImmediate& RHICmdList);
	bool Test_BindlessAccessibleUniformBuffer(FRHICommandListImmediate& RHICmdList);
}
