// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIBufferTests.h"

bool FRHIBufferTests::VerifyBufferContents(const TCHAR* TestName, FRHICommandListImmediate& RHICmdList, TArrayView<FRHIBuffer*> Buffers, TFunctionRef<bool(int32 BufferIndex, void* Ptr, uint32 NumBytes)> VerifyCallback)
{
	TArray<FStagingBufferRHIRef, TInlineAllocator<1>> StagingBuffers;

	for (FRHIBuffer* Buffer : Buffers)
	{
		FStagingBufferRHIRef StagingBuffer = RHICreateStagingBuffer();
		RHICmdList.CopyToStagingBuffer(Buffer, StagingBuffer, 0, Buffer->GetSize());
		StagingBuffers.Emplace(StagingBuffer);
	}

	// @todo - readback API is inconsistent across RHIs
	constexpr bool bFlushResources = true;
	RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

	ON_SCOPE_EXIT
	{
		// Immediate flush to clean up the staging buffer / other resources
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
	};

	for (int32 Index = 0; Index < Buffers.Num(); ++Index)
	{
		FRHIBuffer* Buffer = Buffers[Index];
		FRHIStagingBuffer* StagingBuffer = StagingBuffers[Index];
		uint32 NumBytes = Buffer->GetSize();

		void* Memory = RHICmdList.LockStagingBuffer(StagingBuffer, 0, NumBytes);
		bool Result = VerifyCallback(Index, Memory, NumBytes);
		RHICmdList.UnlockStagingBuffer(StagingBuffer);

		if (!Result)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed (buffer index %d) \"%ls\"", Index, TestName);
			return false;
		}
	}

	UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"%ls\"", TestName);
	return true;
}

static FString HexDump(uint8 const* Ptr, uint32 NumBytes)
{
	FString Result = 
		TEXT("          |  00 01 02 03 04 05 06 07 08 09 0A 0B 0C 0D 0E 0F \n")
		TEXT("   ------ | -------------------------------------------------\n");

	for (uint32 i = 0; i < NumBytes; )
	{
		Result += FString::Printf(TEXT("   0x%04x | "), i);

		for (uint32 j = 0; j < 16 && i < NumBytes; ++j)
		{
			Result += FString::Printf(TEXT(" %02x"), Ptr[i++]);
		}

		Result += TEXT("\n");
	}
	return Result;
}

// Copies data in the specified vertex buffer back to the CPU, and passes a pointer to that data to the provided verification lambda.
bool FRHIBufferTests::VerifyBufferContents(const TCHAR* TestName, FRHICommandListImmediate& RHICmdList, FRHIBuffer* Buffer, TConstArrayView<uint8> ExpectedData)
{
	bool Result;
	{
		uint32 NumBytes = Buffer->GetSize();

		FStagingBufferRHIRef StagingBuffer = RHICreateStagingBuffer();
		RHICmdList.CopyToStagingBuffer(Buffer, StagingBuffer, 0, NumBytes);

		// @todo - readback API is inconsistent across RHIs
		constexpr bool bFlushResources = true;
		RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

		void* Memory = RHICmdList.LockStagingBuffer(StagingBuffer, 0, NumBytes);

		check(ExpectedData.Num() == NumBytes);
		Result = FMemory::Memcmp(ExpectedData.GetData(), Memory, NumBytes) == 0;

		if (!Result)
		{
			FString ExpectedStr = HexDump(ExpectedData.GetData(), NumBytes);
			FString ActualStr = HexDump(static_cast<uint8*>(Memory), NumBytes);

			UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"%ls\"\n\nExpected Data:\n%ls\n\nActual Data:\n%ls\n\n", TestName, *ExpectedStr, *ActualStr);
		}
		else
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"%ls\"", TestName);
		}

		RHICmdList.UnlockStagingBuffer(StagingBuffer);
	}

	// Immediate flush to clean up the staging buffer / other resources
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

	return Result;
}
