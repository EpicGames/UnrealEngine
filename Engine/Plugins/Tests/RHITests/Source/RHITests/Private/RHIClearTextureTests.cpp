// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIClearTextureTests.h"
#include "RHITestsCommon.h"
#include "CommonRenderResources.h"
#include "RenderCaptureInterface.h"
#include "RHIStaticStates.h"
#include "VolumeRendering.h"
#include "IRenderCaptureProvider.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "DummyViewport.h"

// UE_DISABLE_OPTIMIZATION
constexpr bool GSerialTextureTestTask = false; // Set it to true to do map/test/unmap in the render thread instead of a task thread
constexpr bool GBreakOnVerifyFailed = false; // Set it to true to UE_DEBUG_BREAK when a pixel is detected as different
constexpr bool GBreakOnTestEnd = false; // Set it to true to UE_DEBUG_BREAK at the end of the test
constexpr bool GValidateDrawAtlasing = false; // Set it to true to verify that DrawTextureTo2DAtlas output properly the various texture formats.
constexpr bool GLogTestPassed = false; // Set it to true to log every texture clear success
constexpr bool GGPUCaptureTest = false; // Set it to true to trigger a gpu capture surrounding the test. The module needs to be loaded
constexpr bool GBeginEndCaptureHack = false; // Set it to true in case the rhi doesn't support IRenderCaptureProvider::Get().Begin/EndCapture . The RHI needs to support passing a null viewport and require local changes
constexpr uint32 GMaxStagingTexturesToAllocate = 40; // More tasks get spawned as this value increases. This will allocate 576x256 textures. 576 = 32x6x3 to be able to test 3 slices of cubemap arrays

constexpr bool GUseCustomSRV = true; // Set it to false to use the texture directly

// Set this to 1 to get GPU tags which can be helpful when used with GGPUCaptureTest=true
#define CLEAR_TEST_GPU_TAGS 0
// Set this to 1 to get markers in insights
#define CLEAR_TEST_CPU_TAGS 0

#if CLEAR_TEST_GPU_TAGS
#define CLEAR_TEST_SCOPED_DRAW_EVENT(...) SCOPED_DRAW_EVENT(__VA_ARGS__)
#define CLEAR_TEST_SCOPED_DRAW_EVENTF(...) SCOPED_DRAW_EVENTF(__VA_ARGS__)
#else
#define CLEAR_TEST_SCOPED_DRAW_EVENT(...)
#define CLEAR_TEST_SCOPED_DRAW_EVENTF(...)
#endif

#if CLEAR_TEST_CPU_TAGS
#define CLEAR_TEST_SCOPED_NAMED_EVENT_F(...) SCOPED_NAMED_EVENT_F(__VA_ARGS__)
#define CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT(...) SCOPED_NAMED_EVENT_TEXT(__VA_ARGS__)
#else
#define CLEAR_TEST_SCOPED_NAMED_EVENT_F(...)
#define CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT(...)
#endif

static void Test_ClearTextureBeginCapture(FRHICommandListImmediate& RHICmdList)
{
	if (GBeginEndCaptureHack)
	{
		RHICmdList.EndDrawingViewport(nullptr, FRHIPresentArgs(/*.FrameCounter*/ GFrameCounterRenderThread, /*.bPresent*/ true, /*.bLockToVsync*/ false));
		GDynamicRHI->RHIWaitForFlip(100000);
		IRenderCaptureProvider::Get().CaptureFrame();
 		GDynamicRHI->RHISignalFlipEvent();
 		GDynamicRHI->RHIWaitForFlip(100000);
	}
	else
	{
		IRenderCaptureProvider::Get().BeginCapture(&RHICmdList);
	}

}

static void Test_ClearTextureEndCapture(FRHICommandListImmediate& RHICmdList)
{
	if (GBeginEndCaptureHack)
	{
		RHICmdList.EndDrawingViewport(nullptr, FRHIPresentArgs(/*.FrameCounter*/ GFrameCounterRenderThread, /*.bPresent*/ true, /*.bLockToVsync*/ false));
		GDynamicRHI->RHIWaitForFlip(100000);
	}
	else
	{
		IRenderCaptureProvider::Get().EndCapture(&RHICmdList);
	}
}

struct FStagingData
{
	FTextureRHIRef StagingTexture;
	int32 MappedWidth = 0;
	int32 MappedHeight = 0;
	void* MappedPtr = nullptr;
};

struct FMappedPixel
{
	TArrayView<uint8> PixelData;
	uint32 NumChannels;
	uint32 BytesPerChannel;

	FMappedPixel(uint8* Ptr, EPixelFormat PixelFormat, uint32 MappedWidth, uint32 MappedHeight, const FIntVector2& AtlasViewport)
	{
		const FPixelFormatInfo& PixelFormatInfo = GPixelFormats[PixelFormat];
		uint32 BytesPerPixel = PixelFormatInfo.BlockBytes;
		NumChannels = PixelFormatInfo.NumComponents;
		check((BytesPerPixel % NumChannels) == 0);
		BytesPerChannel = BytesPerPixel / NumChannels;
		PixelData = MakeArrayView(Ptr, MappedWidth * BytesPerPixel * AtlasViewport.Y);
	}

	void MoveForward(uint32 BytesPerPixel)
	{
		PixelData = PixelData.RightChop(BytesPerPixel);
	}

	const uint8* GetChannelData(int32 ChannelIndex) const
	{
		return PixelData.GetData() + ChannelIndex * BytesPerChannel;
	}
};

// Description of a single test operation, ie a clear on a given texture / mip / slice. Multiple FTestOperation can point to the same SourceTexture
struct FTestOperation
{
	FTextureRHIRef SourceTexture; // The texture to test
	FStagingData DrawResult;
	FStagingData ClearResult;
	int32 TestMipIndex; // The mips we want to clear
	int32 TestArrayIndex; // The slice we want to clear. If -1, we clear all slices
};

// Here to limit the overhead of creating staging textures every time
struct FStagingTexturePool
{
	FTextureRHIRef CreateTexture2DAtlasStagingFromPool(FRHICommandListImmediate& RHICmdList, EPixelFormat PixelFormat);
	FTextureRHIRef CreateTexture2DAtlasRTFromPool(FRHICommandListImmediate& RHICmdList, EPixelFormat PixelFormat, FUnorderedAccessViewRHIRef* OutUAV);
	void ReturnToPool(const TArray<FTestOperation>& TestOperations);
	void FlushStagingTextures(FRHICommandListImmediate& RHICmdList, EPixelFormat PixelFormat);
	void FlushAllStagingTextures(FRHICommandListImmediate& RHICmdList);
	void PreallocateStagingTexture(FRHICommandListBase& RHICmdList, EPixelFormat PixelFormat, uint32 NumTextures);

private:
	void Recycle(FRHICommandListImmediate& RHICmdList);

	TArray<FTextureRHIRef> CurrentStagingTextures[PF_MAX];
	TArray<FTextureRHIRef> AvailableStagingTextures[PF_MAX];
	FTextureRHIRef StagingTexturePoolCache[PF_MAX] = {};
	FUnorderedAccessViewRHIRef StagingTexturePoolCacheUAV[PF_MAX] = {};
	TArray<FStagingData> TexturesToReturnToPool;
	FRWLock TexturesToReturnToPoolRWLock;
	FGraphEventRef WaitForTextureEvent;
};

FRHITextureCreateDesc CreateTexture2DAtlasDesc(EPixelFormat PixelFormat, ERHIAccess InitialState, ETextureCreateFlags InFlags, const TCHAR* DebugName)
{
	FRHITextureCreateDesc RHITextureCreateDescAtlas = FRHITextureCreateDesc::Create2D(DebugName, 576, 256, PixelFormat)
		.SetFlags(InFlags | TexCreate_NoFastClear | TexCreate_DisableDCC)
		.SetInitialState(InitialState)
		.SetClearValue(EClearBinding::ENoneBound)
		;

	return RHITextureCreateDescAtlas;
}

FTextureRHIRef CreateTexture2DAtlasRT(FRHICommandListBase& RHICmdList, EPixelFormat PixelFormat, FUnorderedAccessViewRHIRef* OutUAV)
{
	FTextureRHIRef Texture2DAtlasRT = RHICmdList.CreateTexture(CreateTexture2DAtlasDesc(PixelFormat, ERHIAccess::UAVCompute, TexCreate_UAV | TexCreate_ShaderResource, TEXT("Texture2DAtlasRT")));
	*OutUAV = RHICmdList.CreateUnorderedAccessView(Texture2DAtlasRT, FRHIViewDesc::CreateTextureUAV().SetDimensionFromTexture(Texture2DAtlasRT));
	(*OutUAV)->SetOwnerName(TEXT("Texture2DAtlasRT"));

	return Texture2DAtlasRT;
}

FTextureRHIRef CreateTexture2DAtlasStaging(FRHICommandListBase& RHICmdList, EPixelFormat PixelFormat)
{
	return RHICmdList.CreateTexture(CreateTexture2DAtlasDesc(PixelFormat, ERHIAccess::CPURead, TexCreate_CPUReadback, TEXT("Texture2DAtlasStaging")));
}

void FStagingTexturePool::PreallocateStagingTexture(FRHICommandListBase& RHICmdList, EPixelFormat PixelFormat, uint32 NumTextures)
{
	CLEAR_TEST_SCOPED_NAMED_EVENT_F(TEXT("AllocateStaging: %d in flight"), FColor::Magenta, CurrentStagingTextures[PixelFormat].Num());
	CurrentStagingTextures[PixelFormat].Reserve(NumTextures);
	AvailableStagingTextures[PixelFormat].Reserve(NumTextures);
	for (uint32 TextureIndex=0; TextureIndex < NumTextures; ++TextureIndex)
	{
		FTextureRHIRef Texture = CreateTexture2DAtlasStaging(RHICmdList, PixelFormat);
		CurrentStagingTextures[PixelFormat].Add(Texture);
		AvailableStagingTextures[PixelFormat].Add(Texture);
	}
}


FTextureRHIRef FStagingTexturePool::CreateTexture2DAtlasStagingFromPool(FRHICommandListImmediate& RHICmdList, EPixelFormat PixelFormat)
{
	check(WaitForTextureEvent == nullptr);

	FGraphEventRef WaitForTextureEventLocal;
	{
		FRWScopeLock WriteLock(TexturesToReturnToPoolRWLock, SLT_Write);
		Recycle(RHICmdList);
		if (AvailableStagingTextures[PixelFormat].Num() == 0)
		{
			check(WaitForTextureEvent == nullptr);
			WaitForTextureEventLocal = FGraphEvent::CreateGraphEvent();
            // Have the text task in-flight warn us when a new texture is ready
			WaitForTextureEvent = WaitForTextureEventLocal;
		}
	}

	if (WaitForTextureEventLocal)
	{
		CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT("WaitForTexture", FColor::Magenta);
		WaitForTextureEventLocal->Wait();
		{
			FRWScopeLock WriteLock(TexturesToReturnToPoolRWLock, SLT_Write);
			Recycle(RHICmdList);
			check(WaitForTextureEvent == nullptr);
		}
	}

	check(AvailableStagingTextures[PixelFormat].Num() > 0);

	FTextureRHIRef NewTexture = AvailableStagingTextures[PixelFormat].Pop();
	RHICmdList.Transition(FRHITransitionInfo(NewTexture, ERHIAccess::CPURead, ERHIAccess::CopyDest));
	return NewTexture;
}

FTextureRHIRef FStagingTexturePool::CreateTexture2DAtlasRTFromPool(FRHICommandListImmediate& RHICmdList, EPixelFormat PixelFormat, FUnorderedAccessViewRHIRef* OutUAV)
{
	if (StagingTexturePoolCache[PixelFormat] == nullptr)
	{
		StagingTexturePoolCache[PixelFormat] = CreateTexture2DAtlasRT(RHICmdList, PixelFormat, &StagingTexturePoolCacheUAV[PixelFormat]);
	}

	*OutUAV = StagingTexturePoolCacheUAV[PixelFormat];
	return StagingTexturePoolCache[PixelFormat];
}

void FStagingTexturePool::FlushAllStagingTextures(FRHICommandListImmediate& RHICmdList)
{
	for (uint32 PixelFormat = 0; PixelFormat < PF_MAX; ++PixelFormat)
	{
		FlushStagingTextures(RHICmdList, (EPixelFormat)PixelFormat);
	}
}


void FStagingTexturePool::FlushStagingTextures(FRHICommandListImmediate& RHICmdList, EPixelFormat PixelFormat)
{
	FRWScopeLock WriteLock(TexturesToReturnToPoolRWLock, SLT_Write);
	Recycle(RHICmdList);
	CurrentStagingTextures[PixelFormat].Empty();
	AvailableStagingTextures[PixelFormat].Empty();
}

void MapStagingSurfaces(FRHICommandListImmediate& RHICmdList, FStagingData& StagingData)
{
	if (StagingData.StagingTexture)
	{
		RHICmdList.MapStagingSurface(StagingData.StagingTexture, nullptr /*Fence*/, StagingData.MappedPtr, StagingData.MappedWidth, StagingData.MappedHeight);
	}
}

void UnmapStagingSurfaces(FRHICommandListImmediate& RHICmdList, FStagingData& StagingData)
{
	if (StagingData.StagingTexture)
	{
		RHICmdList.UnmapStagingSurface(StagingData.StagingTexture);
		StagingData.StagingTexture = nullptr;
	}
}

void FStagingTexturePool::Recycle(FRHICommandListImmediate& RHICmdList)
{
	for (FStagingData& TextureToReturnToPool : TexturesToReturnToPool)
	{
		FTextureRHIRef StagingTexture = TextureToReturnToPool.StagingTexture;
		EPixelFormat PixelFormat = StagingTexture->GetDesc().Format;
		UnmapStagingSurfaces(RHICmdList, TextureToReturnToPool);
		AvailableStagingTextures[PixelFormat].Add(StagingTexture);
	}
	TexturesToReturnToPool.Empty();
}

void FStagingTexturePool::ReturnToPool(const TArray<FTestOperation>& TestOperations)
{
	FRWScopeLock WriteLock(TexturesToReturnToPoolRWLock, SLT_Write);
	for (const FTestOperation& TestOperation : TestOperations)
	{
		if (TestOperation.DrawResult.StagingTexture)
		{
			TexturesToReturnToPool.Add(TestOperation.DrawResult);
		}

		if (TestOperation.ClearResult.StagingTexture)
		{
			TexturesToReturnToPool.Add(TestOperation.ClearResult);
		}
	}

	if (WaitForTextureEvent)
	{
		WaitForTextureEvent->DispatchSubsequents();
        // We must call DispatchSubsequents only once
		WaitForTextureEvent = nullptr;
	}

}

static FString GetTextureName(const FRHITextureDesc& Desc)
{
	FString TextureSuffix = Desc.IsTexture3D() ? FString::Printf(TEXT("x%d"), Desc.Depth) : Desc.IsTextureArray() ? FString::Printf(TEXT("x%d"), Desc.ArraySize) : TEXT("");
	bool bIndependentRTVPerSlice = EnumHasAnyFlags(Desc.Flags, TexCreate_TargetArraySlicesIndependently);
	FString SliceDesc = bIndependentRTVPerSlice ? TEXT("1 / Slice") : TEXT("All Slices");
	return FString::Printf(TEXT("(%s %dx%d%s %d mips, 0x%llX) %s %s, %s"), GetTextureDimensionString(Desc.Dimension), Desc.Extent.X, Desc.Extent.Y, *TextureSuffix, Desc.NumMips, EnumToUnderlyingType(Desc.Flags), GPixelFormats[Desc.Format].Name,
		*Desc.ClearValue.GetClearColor().ToString(), *SliceDesc);
}

enum ETestStage
{
	ETestStage_Draw,
	ETestStage_Clear,
	ETestStage_Unknown,
};

struct FTestStageFailure
{
	int32 MipIndex = -2;
	int32 SliceIndex = -2;
	ETestStage TestStage = ETestStage_Unknown;
};

struct FTestContext
{
	FVertexDeclarationRHIRef VertexDeclarationRHI;
	FVector4f SentinelColor;
	FStagingTexturePool StagingTexturePool;
	TArray<int32> TextureNumsClearsSuccess;
	TArray<int32> TextureNumClears;
	TArray<FGraphEventRef> AllEvents;
	uint32 BatchStartTextureIndex;
	TArray<FRHITextureCreateDesc> RHITextureCreateDescs;
	TArray<FTestStageFailure> TestStageFailures;

	FTestContext(FVertexDeclarationRHIRef InVertexDeclarationRHI, const TArray<FRHITextureCreateDesc>& InRHITextureCreateDescs, const FVector4f& InSentinelColor)
	{
		VertexDeclarationRHI = InVertexDeclarationRHI;
		RHITextureCreateDescs = InRHITextureCreateDescs;
		SentinelColor = InSentinelColor;
		TextureNumsClearsSuccess.SetNum(RHITextureCreateDescs.Num());
		TextureNumClears.SetNum(RHITextureCreateDescs.Num());
		TestStageFailures.SetNum(RHITextureCreateDescs.Num());
		for (int32 TextureIndex = 0; TextureIndex < RHITextureCreateDescs.Num(); ++TextureIndex)
		{
			TextureNumsClearsSuccess[TextureIndex] = 0;
			TextureNumClears[TextureIndex] = 0;
		}
	}

	void SetExpectedNumClears(uint32 TextureIndex, uint32 ExpectedNumClears)
	{
		TextureNumClears[TextureIndex + BatchStartTextureIndex] = ExpectedNumClears;
	}

	void SetClearTestResult(uint32 GlobalTextureIndex, int32 NumClearSuccess, const FTestStageFailure& TestStageFailure)
	{
		if (TestStageFailures[GlobalTextureIndex].TestStage == ETestStage_Unknown && TestStageFailure.TestStage != ETestStage_Unknown)
		{
			TestStageFailures[GlobalTextureIndex] = TestStageFailure;
		}

		if (NumClearSuccess > 0)
		{
			int32 CurrentTaskClearSuccess = NumClearSuccess + FPlatformAtomics::InterlockedAdd(&TextureNumsClearsSuccess[GlobalTextureIndex], NumClearSuccess);
			check(CurrentTaskClearSuccess <= TextureNumClears[GlobalTextureIndex]);
			if (CurrentTaskClearSuccess == TextureNumClears[GlobalTextureIndex] && GLogTestPassed)
			{
 				UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. Test_ClearTexture \"%ls (%d clears)\"",
 					*GetTextureName(RHITextureCreateDescs[GlobalTextureIndex]), TextureNumClears[GlobalTextureIndex]);
			}
		}
	}
};

int32 ComputeNumSlices(const FRHITextureDesc& Desc, uint32 MipIndex)
{
	uint32 MipDepth = FMath::Max(Desc.Depth >> MipIndex, 1);
	int32 NumSlices = (Desc.Dimension != ETextureDimension::Texture3D) ? Desc.ArraySize : MipDepth;
	if (Desc.IsTextureCube())
	{
		NumSlices *= 6;
	}
	return NumSlices;
}

FIntVector2 GetAtlasViewport(const FRHITextureDesc& Desc)
{
	uint32 NumSlices = ComputeNumSlices(Desc, 0);
	FIntVector2 AtlasViewport;
	AtlasViewport.X = Desc.Extent.X * NumSlices;
	AtlasViewport.Y = Desc.Extent.Y;
	if (Desc.NumMips > 1)
	{
		AtlasViewport.Y *= 2;

	}
	return AtlasViewport;
}

using FVerifyDataCallback = TFunctionRef<bool(const FMappedPixel& Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 MipIndex, uint32 SliceIndex, EPixelFormat PixelFormat)>;

bool VerifyTextureData(FStagingData& StagingData, const FRHITexture* SourceTexture, FVerifyDataCallback VerifyCallback)
{
	const FRHITextureDesc& SourceDesc = SourceTexture->GetDesc();
	const FRHITextureDesc& StagingDesc = StagingData.StagingTexture->GetDesc();

	FRHICopyTextureInfo CopyInfo;
	FIntVector Size = SourceTexture->GetSizeXYZ();

	bool bResult = true;
	uint32 BytesPerPixel = GPixelFormats[StagingDesc.Format].BlockBytes;

	FMappedPixel MappedRow((uint8*)StagingData.MappedPtr, StagingDesc.Format, StagingData.MappedWidth, StagingData.MappedHeight, GetAtlasViewport(SourceDesc));

	for (int32 MipIndex = SourceDesc.NumMips - 1; MipIndex >= 0 ; --MipIndex)
	{
		uint32 MipWidth = FMath::Max(Size.X >> MipIndex, 1);
		uint32 MipHeight = FMath::Max(Size.Y >> MipIndex, 1);
		uint32 MipDepth = FMath::Max(Size.Z >> MipIndex, 1);

		CopyInfo.DestPosition.X = 0;

		int32 NumSlices = ComputeNumSlices(SourceDesc, MipIndex);

		FMappedPixel MappedPixel = MappedRow;
		for (int32 SliceIndex = 0; SliceIndex < NumSlices; ++SliceIndex)
		{
			if (!VerifyCallback(MappedPixel, MipWidth, MipHeight, StagingData.MappedWidth, MipIndex, SliceIndex, SourceDesc.Format))
			{
				if (GBreakOnVerifyFailed)
				{
					UE_DEBUG_BREAK();
				}

				bResult = false;
			}
			MappedPixel.MoveForward(MipWidth * BytesPerPixel);
		}

		MappedRow.MoveForward(MipHeight * StagingData.MappedWidth * BytesPerPixel);
	}

	return bResult;
}

class FSimpleDrawVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FSimpleDrawVS);
	FSimpleDrawVS() = default;
	FSimpleDrawVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}
};
IMPLEMENT_GLOBAL_SHADER(FSimpleDrawVS, "/Plugin/RHITests/Private/TestFillTexture.usf", "TestFillTextureVS", SF_Vertex);

class FSimpleDrawPSBase : public FGlobalShader
{
	DECLARE_TYPE_LAYOUT(FSimpleDrawPSBase, NonVirtual);
public:
	FSimpleDrawPSBase() = default;
	FSimpleDrawPSBase(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FGlobalShader(Initializer)
	{
		TestFillTextureConstant.Bind(Initializer.ParameterMap, TEXT("TestFillTextureConstant"), SPF_Mandatory);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}

	static TShaderRef<FSimpleDrawPSBase> SelectShader(FGlobalShaderMap* GlobalShaderMap, EPixelFormat PixelFormat);

	static const TCHAR* GetSourceFilename() { return TEXT("/Plugin/RHITests/Private/TestFillTexture.usf"); }
	static const TCHAR* GetFunctionName() { return TEXT("TestFillTexturePS"); }

	LAYOUT_FIELD(FShaderParameter, TestFillTextureConstant);
};

IMPLEMENT_TYPE_LAYOUT(FSimpleDrawPSBase);

template<EPixelFormat PixelFormatType>
class TSimpleDrawPS : public FSimpleDrawPSBase
{
	DECLARE_SHADER_TYPE(TSimpleDrawPS, Global);
public:
	TSimpleDrawPS() = default;
	TSimpleDrawPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FSimpleDrawPSBase(Initializer) {}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FSimpleDrawPSBase::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetRenderTargetOutputFormat(0, PixelFormatType);
	}
};
IMPLEMENT_SHADER_TYPE(template<>, TSimpleDrawPS<PF_R8G8B8A8>, FSimpleDrawPSBase::GetSourceFilename(), FSimpleDrawPSBase::GetFunctionName(), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, TSimpleDrawPS<PF_FloatRGBA>, FSimpleDrawPSBase::GetSourceFilename(), FSimpleDrawPSBase::GetFunctionName(), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, TSimpleDrawPS<PF_A32B32G32R32F>, FSimpleDrawPSBase::GetSourceFilename(), FSimpleDrawPSBase::GetFunctionName(), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, TSimpleDrawPS<PF_A16B16G16R16>, FSimpleDrawPSBase::GetSourceFilename(), FSimpleDrawPSBase::GetFunctionName(), SF_Pixel);
IMPLEMENT_SHADER_TYPE(template<>, TSimpleDrawPS<PF_R16G16B16A16_UNORM>, FSimpleDrawPSBase::GetSourceFilename(), FSimpleDrawPSBase::GetFunctionName(), SF_Pixel);


TShaderRef<FSimpleDrawPSBase> FSimpleDrawPSBase::SelectShader(FGlobalShaderMap* GlobalShaderMap, EPixelFormat PixelFormat)
{
	switch (PixelFormat)
	{
	default: checkNoEntry();
	case PF_R8G8B8A8:	   return  TShaderMapRef<TSimpleDrawPS<PF_R8G8B8A8> >(GlobalShaderMap);
	case PF_FloatRGBA:	   return  TShaderMapRef<TSimpleDrawPS<PF_FloatRGBA> >(GlobalShaderMap);
	case PF_A32B32G32R32F:	   return  TShaderMapRef<TSimpleDrawPS<PF_A32B32G32R32F> >(GlobalShaderMap);
	case PF_A16B16G16R16:	   return  TShaderMapRef<TSimpleDrawPS<PF_A16B16G16R16> >(GlobalShaderMap);
	case PF_R16G16B16A16_UNORM:	   return  TShaderMapRef<TSimpleDrawPS<PF_R16G16B16A16_UNORM> >(GlobalShaderMap);
	}
}


// Pixel shader to composite UI over HDR buffer
class FTestTextureToAtlasBase : public FGlobalShader
{
	DECLARE_TYPE_LAYOUT(FTestTextureToAtlasBase, NonVirtual);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	FTestTextureToAtlasBase(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FGlobalShader(Initializer)
	{
		MipBiasMipNumsViewport.Bind(Initializer.ParameterMap, TEXT("MipBiasMipNumsViewport"));
		SrcResourceParam.Bind(Initializer.ParameterMap, TEXT("SrcResource"));
		RWAtlas2D.Bind(Initializer.ParameterMap, TEXT("RWAtlas2D"));
	}
	FTestTextureToAtlasBase() = default;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("TEXTURE_TO_ATLAS"), 1);
	}

	static TShaderRef<FTestTextureToAtlasBase> SelectShader(FGlobalShaderMap* GlobalShaderMap, ETextureDimension TextureDimension, EPixelFormat PixelFormat);

	static const TCHAR* GetSourceFilename() { return TEXT("/Plugin/RHITests/Private/TestFillTexture.usf"); }
	static const TCHAR* GetFunctionName() { return TEXT("TestTextureToAtlasCS"); }

	LAYOUT_FIELD(FShaderParameter, MipBiasMipNumsViewport);
	LAYOUT_FIELD(FShaderResourceParameter, SrcResourceParam);
	LAYOUT_FIELD(FShaderResourceParameter, RWAtlas2D);
};

IMPLEMENT_TYPE_LAYOUT(FTestTextureToAtlasBase);

template<ETextureDimension SrcType>
class TTestTextureToAtlasCS : public FTestTextureToAtlasBase
{
	DECLARE_SHADER_TYPE(TTestTextureToAtlasCS, Global);
public:
	TTestTextureToAtlasCS() = default;
	TTestTextureToAtlasCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer) : FTestTextureToAtlasBase(Initializer) {}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FTestTextureToAtlasBase::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("SRC_TYPE"), uint32(SrcType));

	}
};

// The using are here because IMPLEMENT_SHADER_TYPE can't take comma separated template parameters
using FTestTextureToAtlasCS_2D= TTestTextureToAtlasCS<ETextureDimension::Texture2D>;
IMPLEMENT_SHADER_TYPE(template<>, FTestTextureToAtlasCS_2D, FTestTextureToAtlasBase::GetSourceFilename(), FTestTextureToAtlasBase::GetFunctionName(), SF_Compute);

using FTestTextureToAtlasCS_2DArray = TTestTextureToAtlasCS<ETextureDimension::Texture2DArray>;
IMPLEMENT_SHADER_TYPE(template<>, FTestTextureToAtlasCS_2DArray, FTestTextureToAtlasBase::GetSourceFilename(), FTestTextureToAtlasBase::GetFunctionName(), SF_Compute);

using FTestTextureToAtlasCS_3D = TTestTextureToAtlasCS<ETextureDimension::Texture3D>;
IMPLEMENT_SHADER_TYPE(template<>, FTestTextureToAtlasCS_3D, FTestTextureToAtlasBase::GetSourceFilename(), FTestTextureToAtlasBase::GetFunctionName(), SF_Compute);

TShaderRef<FTestTextureToAtlasBase> FTestTextureToAtlasBase::SelectShader(FGlobalShaderMap* GlobalShaderMap, ETextureDimension TextureDimension, EPixelFormat PixelFormat)
{
	switch (TextureDimension)
	{
	default: checkNoEntry();
	case ETextureDimension::Texture2D:	   return  TShaderMapRef<FTestTextureToAtlasCS_2D>(GlobalShaderMap);
	case ETextureDimension::TextureCube:
	case ETextureDimension::TextureCubeArray:
	case ETextureDimension::Texture2DArray: return TShaderMapRef<FTestTextureToAtlasCS_2DArray>(GlobalShaderMap);
	case ETextureDimension::Texture3D:      return TShaderMapRef<FTestTextureToAtlasCS_3D>(GlobalShaderMap);
	}
}

static void DrawTextureTo2DAtlas(FRHICommandListImmediate& RHICmdList, FRHITexture* SourceTexture, FRHIShaderResourceView* SourceSRV, FRHITexture* DestTexture, FRHIUnorderedAccessView* DestTextureUAV, FRHIVertexDeclaration* VertexDeclarationRHI)
{
	CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT("Test_ClearTexture_DrawTextureTo2DAtlas", FColor::Magenta);
	TShaderRef<FTestTextureToAtlasBase> ComputeShader = FTestTextureToAtlasBase::SelectShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), SourceTexture->GetDesc().Dimension, SourceTexture->GetDesc().Format);
	
	uint32 SourceNumMips = SourceTexture->GetDesc().NumMips;
	uint32 SourceMipBias = SourceNumMips > 1 ? 0 : FMath::FloorLog2(SourceTexture->GetDesc().Extent.X);

	FIntVector2 AtlasViewport = GetAtlasViewport(SourceTexture->GetDesc());
	FUintVector4 MipBiasMipNumsViewport(SourceMipBias, SourceNumMips, AtlasViewport.X, AtlasViewport.Y);

	check(DestTexture->GetDesc().Extent.X >= AtlasViewport.X);
	check(DestTexture->GetDesc().Extent.Y >= AtlasViewport.Y);

	SetComputePipelineState(RHICmdList, ComputeShader.GetComputeShader());
	FRHIBatchedShaderParameters& ShaderParameters = RHICmdList.GetScratchShaderParameters();
	SetShaderValue(ShaderParameters, ComputeShader->MipBiasMipNumsViewport, MipBiasMipNumsViewport);
	if (GUseCustomSRV)
	{
		SetSRVParameter(ShaderParameters, ComputeShader->SrcResourceParam, SourceSRV);
	}
	else
	{
		SetTextureParameter(ShaderParameters, ComputeShader->SrcResourceParam, SourceTexture); // this doesn't work for cubemaps/cubemaps arrays when doing .Load on a Texture2DArray
	}
	SetUAVParameter(ShaderParameters, ComputeShader->RWAtlas2D, DestTextureUAV);
	RHICmdList.SetBatchedShaderParameters(ComputeShader.GetComputeShader(), ShaderParameters);
	RHICmdList.DispatchComputeShader((AtlasViewport.X + 7) / 8, (AtlasViewport.Y + 7) / 8, 1);
}

// Assumes IntermediateTexture is in RTV state, and DestTexture in CopyDest
static void CopyTextureToUnpackedStaging(FRHICommandListImmediate& RHICmdList, FRHITexture* SourceTexture, FRHIShaderResourceView* SourceSRV, FRHITexture* IntermediateTexture, FRHIUnorderedAccessView* IntermediateUAV, FRHITexture* DestTexture, FRHIVertexDeclaration* VertexDeclarationRHI)
{
	CLEAR_TEST_SCOPED_DRAW_EVENT(RHICmdList, CopyTextureToUnpackedStaging);

	DrawTextureTo2DAtlas(RHICmdList, SourceTexture, SourceSRV, IntermediateTexture, IntermediateUAV, VertexDeclarationRHI);
	RHICmdList.Transition(FRHITransitionInfo(IntermediateTexture, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));

	FRHICopyTextureInfo RHICopyTextureInfo;
	FIntVector2 AtlasViewport = GetAtlasViewport(SourceTexture->GetDesc());
// 	RHICopyTextureInfo.Size.X = AtlasViewport.X;
// 	RHICopyTextureInfo.Size.Y = AtlasViewport.Y;
// 	RHICopyTextureInfo.Size.Z = AtlasViewport.Y;

	RHICmdList.CopyTexture(IntermediateTexture, DestTexture, RHICopyTextureInfo);
	RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::CopyDest, ERHIAccess::CPURead));
	RHICmdList.Transition(FRHITransitionInfo(IntermediateTexture, ERHIAccess::CopySrc, ERHIAccess::UAVCompute));
}

static void ConvertColorToUNORM16(uint8 OutTextureColor[16], const FLinearColor& LinearColor)
{
	uint16 ColorUNorm16[] = 
	{
		(uint16)(0.5f + FLinearColor::Clamp01NansTo0(LinearColor.R) * 65535.f),
		(uint16)(0.5f + FLinearColor::Clamp01NansTo0(LinearColor.G) * 65535.f),
		(uint16)(0.5f + FLinearColor::Clamp01NansTo0(LinearColor.B) * 65535.f),
		(uint16)(0.5f + FLinearColor::Clamp01NansTo0(LinearColor.A) * 65535.f)
	};

	FMemory::Memcpy(OutTextureColor, ColorUNorm16, sizeof(ColorUNorm16));
}

void ConvertLinearColorToTextureColor(uint8 OutTextureColor[16], EPixelFormat PixelFormat, const FLinearColor& LinearColor)
{
	const FPixelFormatInfo& PixelFormatInfo = GPixelFormats[PixelFormat];
	switch (PixelFormat)
	{
	case PF_A32B32G32R32F:
		FMemory::Memcpy(OutTextureColor, &LinearColor, sizeof(LinearColor));
		break;

	case PF_A16B16G16R16:
	case PF_R16G16B16A16_UNORM:
		ConvertColorToUNORM16(OutTextureColor, LinearColor);
		break;
	case PF_FloatRGBA:
	{
		FFloat16Color Float16Color(LinearColor);
		FMemory::Memcpy(OutTextureColor, &Float16Color, sizeof(Float16Color));
	}
		break;
	case PF_R8G8B8A8:
	{
		uint32 U32Color = LinearColor.QuantizeRound().ToPackedABGR();
		FMemory::Memcpy(OutTextureColor, &U32Color, sizeof(U32Color));
	}
		break;
	default:
		break;
	}
}

template<class TBinaryValue>
static bool ComparePixels(uint8* ReferenceValue, const FMappedPixel& Pixel, int64 Tolerance, uint32 NumChannels)
{
	TArrayView<TBinaryValue> TestPixelA((TBinaryValue*)ReferenceValue, NumChannels);

	for (uint32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
	{
		TBinaryValue TestPixelB = *((TBinaryValue*)Pixel.GetChannelData(ChannelIndex));
		int64 Difference = (int64)TestPixelA[ChannelIndex] - (int64)TestPixelB;
		if (FMath::Abs(Difference) > Tolerance)
		{
			return false;
		}
	}
	return true;
}

static bool ComparePixelValues(uint8* ReferenceValue, const FMappedPixel& Pixel, EPixelFormat PixelFormat)
{
	switch (PixelFormat)
	{
	case PF_A32B32G32R32F: return ComparePixels<uint32>(ReferenceValue, Pixel, 0, 4);
	case PF_A16B16G16R16: return ComparePixels<uint16>(ReferenceValue, Pixel, 1, 4);
	case PF_R16G16B16A16_UNORM: return ComparePixels<uint16>(ReferenceValue, Pixel, 1, 4);
	case PF_FloatRGBA: return ComparePixels<uint16>(ReferenceValue, Pixel, 1, 4);
	case PF_R8G8B8A8: return ComparePixels<uint8>(ReferenceValue, Pixel, 1, 4);
	default:
		checkNoEntry();
		return false;
	}
}

template<class TBinaryValue>
bool AreTextureColorsEqual(const FLinearColor& ClearColorLinear, EPixelFormat PixelFormat, const FMappedPixel& Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 CurrentMipIndex, uint32 CurrentSliceIndex, int64 Tolerance)
{
	uint32 NumChannels = GPixelFormats[PixelFormat].NumComponents;
	uint8 TextureColor[16];
	ConvertLinearColorToTextureColor(TextureColor, PixelFormat, ClearColorLinear);

	FMappedPixel Row = Ptr;

	EPixelFormat AtlasPixelFormat = PixelFormat;
	uint32 MappedBytesPerPixel = GPixelFormats[AtlasPixelFormat].BlockBytes;

	for (uint32 Y = 0; Y < MipHeight; ++Y)
	{
		FMappedPixel Pixel = Row;

		for (uint32 X = 0; X < MipWidth; ++X)
		{
			if (!ComparePixels<TBinaryValue>(TextureColor, Pixel, Tolerance, NumChannels))
			{
				return false;
			}

			Pixel.MoveForward(MappedBytesPerPixel);
		}
		Row.MoveForward(Width * MappedBytesPerPixel);
	}
	return true;
}

bool AreTextureColorsEqual(const FLinearColor& LinearColor, EPixelFormat PixelFormat, const FMappedPixel& Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 CurrentMipIndex, uint32 CurrentSliceIndex)
{
	switch (PixelFormat)
	{
	case PF_A32B32G32R32F: return AreTextureColorsEqual<uint32>(LinearColor, PixelFormat, Ptr, MipWidth, MipHeight, Width, CurrentMipIndex, CurrentSliceIndex, 0);
	case PF_R16G16B16A16_UNORM: return AreTextureColorsEqual<uint16>(LinearColor, PixelFormat, Ptr, MipWidth, MipHeight, Width,  CurrentMipIndex, CurrentSliceIndex, 1);
	case PF_A16B16G16R16: return AreTextureColorsEqual<uint16>(LinearColor, PixelFormat, Ptr, MipWidth, MipHeight, Width,  CurrentMipIndex, CurrentSliceIndex, 1);
	case PF_FloatRGBA: return AreTextureColorsEqual<uint16>(LinearColor, PixelFormat, Ptr, MipWidth, MipHeight, Width,  CurrentMipIndex, CurrentSliceIndex, 1);
	case PF_R8G8B8A8: return AreTextureColorsEqual<uint8>(LinearColor, PixelFormat, Ptr, MipWidth, MipHeight, Width,  CurrentMipIndex, CurrentSliceIndex, 1);
	default: 
		checkNoEntry();
		return false;
	}
}

bool AllSlicesInRTV(const FRHITextureDesc& Desc)
{
	return (Desc.Dimension == ETextureDimension::Texture3D) ||
		(Desc.Dimension == ETextureDimension::Texture2DArray && !EnumHasAnyFlags(Desc.Flags, TexCreate_TargetArraySlicesIndependently)) ||
		(Desc.Dimension == ETextureDimension::TextureCube && !EnumHasAnyFlags(Desc.Flags, TexCreate_TargetArraySlicesIndependently));
}

static void DrawColoredQuad(FRHICommandListImmediate& RHICmdList, FRHIVertexDeclaration* VertexDeclarationRHI, FRHITexture* RenderTarget, const FVector4f& ConstColor, uint32 TestMipIndex = -1, uint32 TestArrayIndex = -1)
{
	const FRHITextureDesc& Desc = RenderTarget->GetDesc();
	bool bAllSlicesInRTV = AllSlicesInRTV(Desc);
	if (TestArrayIndex != -1)
	{
		check(!bAllSlicesInRTV);
	}

	int32 NumMips =  (TestMipIndex != -1) ? 1 : Desc.NumMips;

	CLEAR_TEST_SCOPED_NAMED_EVENT_F(TEXT("DrawColoredQuad - %s"), FColor::Magenta, *GetTextureName(Desc));

	TShaderRef<FSimpleDrawPSBase> PixelShader = FSimpleDrawPSBase::SelectShader(GetGlobalShaderMap(GMaxRHIFeatureLevel), Desc.Format);

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.PrimitiveType = EPrimitiveType::PT_TriangleStrip;

	// Do not deal with texture arrays of volume textures
	check((Desc.Depth == 1) || (Desc.ArraySize == 1));

	TOptionalShaderMapRef<FWriteToSliceVS> VertexShaderVolume(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	TShaderMapRef<FSimpleDrawVS> VertexShaderSimple(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	if (bAllSlicesInRTV)
	{
		TOptionalShaderMapRef<FWriteToSliceGS> GeometryShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
		GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GScreenVertexDeclaration.VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShaderVolume.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.SetGeometryShader(GeometryShader.GetGeometryShader());
	}
	else
	{
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShaderSimple.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.SetGeometryShader(nullptr);
	}

	int32 StartMipIndex = (TestMipIndex != -1) ? TestMipIndex : 0;
	int32 EndMipIndex = StartMipIndex + NumMips;

	for (int32 MipIndex = StartMipIndex; MipIndex < EndMipIndex; ++MipIndex)
	{
		int32 StartSliceIndex = (TestArrayIndex != -1) ? TestArrayIndex : 0;
		int32 NumSlices = (bAllSlicesInRTV || (TestArrayIndex != -1)) ? 1 : ComputeNumSlices(Desc, MipIndex);
		int32 EndSliceIndex = StartSliceIndex + NumSlices;

		for (int32 SliceIndex = StartSliceIndex; SliceIndex < EndSliceIndex; ++SliceIndex)
		{
			uint32 MipWidth = FMath::Max(Desc.Extent.X >> MipIndex, 1);
			uint32 MipHeight = FMath::Max(Desc.Extent.Y >> MipIndex, 1);

			FRHIRenderPassInfo RenderPassInfo(RenderTarget, ERenderTargetActions::DontLoad_Store, nullptr, MipIndex, bAllSlicesInRTV ? -1 : SliceIndex);

			RHICmdList.BeginRenderPass(RenderPassInfo, TEXT("Test_Clear_DrawColoredQuad"));
			RHICmdList.SetViewport(0, 0, 0, float(MipWidth), float(MipHeight), 1);
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
			SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

			FRHIBatchedShaderParameters& ShaderParameters = RHICmdList.GetScratchShaderParameters();
			SetShaderValue(ShaderParameters, PixelShader->TestFillTextureConstant, ConstColor);
			RHICmdList.SetBatchedShaderParameters(PixelShader.GetPixelShader(), ShaderParameters);

			if (bAllSlicesInRTV)
			{
				FVolumeBounds VolumeBounds(MipWidth);
				VolumeBounds.MaxZ = (Desc.Dimension == ETextureDimension::Texture3D) ? FMath::Max(Desc.Depth >> MipIndex, 1) : Desc.ArraySize;
				SetShaderParametersLegacyVS(RHICmdList, VertexShaderVolume, VolumeBounds, FIntVector(VolumeBounds.MaxX - VolumeBounds.MinX));
				RasterizeToVolumeTexture(RHICmdList, VolumeBounds);
			}
			else
			{
				RHICmdList.DrawPrimitive(0, 1, 1);
			}
			RHICmdList.EndRenderPass();
		}
	}
}

struct FTestBatchRequestIteration
{
	TArray<FTestOperation> TestOperations;
	uint32 CollectTextureOperations(FTextureRHIRef Texture)
	{
		const FRHITextureDesc& Desc = Texture->GetDesc();
		bool bIndependentRTVPerSlice = Desc.Dimension == ETextureDimension::Texture2D || EnumHasAnyFlags(Desc.Flags, TexCreate_TargetArraySlicesIndependently);

		check(Desc.Depth == 1 || Desc.ArraySize == 1);
		uint32 ReadbackWidth = Desc.Extent.X * Desc.Depth * Desc.ArraySize;
		uint32 ReadbackHeight = Desc.Extent.Y * (Desc.NumMips > 1 ? 2 : 1);

		uint32 NumTestOperations = 0;
		if (bIndependentRTVPerSlice)
		{
			// Clear mip/slice individually
			for (int32 TestMipIndex = 0; TestMipIndex < Desc.NumMips; ++TestMipIndex)
			{
				int32 NumSlices = ComputeNumSlices(Desc, TestMipIndex);
				for (int32 TestArrayIndex = 0; TestArrayIndex < NumSlices; ++TestArrayIndex)
				{
					TestOperations.Add(FTestOperation{ Texture , FStagingData(), FStagingData(), TestMipIndex , TestArrayIndex });
					++NumTestOperations;
				}
			}
		}
		else
		{
			if (Desc.IsTexture3D())
			{
				TestOperations.Add(FTestOperation{ Texture , FStagingData(), FStagingData(), 0 , -1  });
				++NumTestOperations;
			}
			else if (Desc.IsTextureArray())
			{
				// some RHIs don't support clearing all slices beyond mip 0. Do not test them for now
				TestOperations.Add(FTestOperation{ Texture , FStagingData(), FStagingData(), 0 , -1 });
				++NumTestOperations;
			}
		}

		return NumTestOperations;
	}
};

class FTextureTestTasks
{
public:
	TArray<FTestOperation> TestOperations;
	FTestContext* TaskTestContext;
	FRHITextureDesc Desc;
	bool bReportResultsOnEnd;
	uint32 TaskTextureIndex;

	FTestStageFailure TestStageFailure;

	void SetTestStageFailure(ETestStage TestStage, int32 MipIndex, int32 SliceIndex)
	{
		if (TestStageFailure.TestStage == ETestStage_Unknown)
		{
			TestStageFailure.MipIndex = MipIndex;
			TestStageFailure.SliceIndex = SliceIndex;
			TestStageFailure.TestStage = TestStage;
		}
	}

	static constexpr ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }
	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FTextureTestTasks, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::AnyHiPriThreadHiPriTask; }

	FTextureTestTasks(const TArrayView<FTestOperation>& InTestOperations, FTestContext& TestContext, const FRHITextureDesc& InDesc, bool bInReportResultsOnEnd)
		: TestOperations(InTestOperations)
		, TaskTestContext(&TestContext)
		, Desc(InDesc)
		, bReportResultsOnEnd(bInReportResultsOnEnd)
		, TaskTextureIndex(TestContext.BatchStartTextureIndex)
	{}

	bool ProcessDrawResults()
	{
		CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT("VerifyTextureData_Draw", FColor::Magenta);
		bool bProcessDrawResults = true;
		for (FTestOperation& TestOperation : TestOperations)
		{
			if (TestOperation.DrawResult.StagingTexture == nullptr) 
				continue;

			bool bTestSuccess = VerifyTextureData(TestOperation.DrawResult, TestOperation.SourceTexture,
				[&](const FMappedPixel& Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 CurrentMipIndex, uint32 CurrentSliceIndex, EPixelFormat PixelFormat)
				{
					if (!AreTextureColorsEqual(TaskTestContext->SentinelColor, TestOperation.SourceTexture->GetDesc().Format, Ptr, MipWidth, MipHeight, Width, CurrentMipIndex, CurrentSliceIndex))
					{
						SetTestStageFailure(ETestStage_Draw, CurrentMipIndex, CurrentSliceIndex);
						return false;
					}
					return true;
				});

			if (!bTestSuccess)
			{
				bProcessDrawResults = false;
				break;
			}
		}

		return bProcessDrawResults;
	}

	uint32 ProcessClearResults()
	{
		CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT("VerifyTextureData_Clear", FColor::Magenta);
		uint32 NumClearSuccess = 0;
		for (FTestOperation& TestOperation : TestOperations)
		{
			if (TestOperation.ClearResult.StagingTexture == nullptr)
				continue;

			bool bTestSuccess = VerifyTextureData(TestOperation.ClearResult, TestOperation.SourceTexture,
				[&](const FMappedPixel& Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 CurrentMipIndex, uint32 CurrentSliceIndex, EPixelFormat PixelFormat)
				{
					FVector4f SelectedColor = TaskTestContext->SentinelColor;
					// make sure that clear only wrote where it's supposed to be and have the sentinel value from the draw still here
					// In case we provided -1 as a slice index, ignore the current slice since we're supposed to have clear all of them
					if (CurrentMipIndex == TestOperation.TestMipIndex && (CurrentSliceIndex == TestOperation.TestArrayIndex || TestOperation.TestArrayIndex == -1))
					{
						SelectedColor = TestOperation.SourceTexture->GetClearColor();
					}

					if (!AreTextureColorsEqual(SelectedColor, TestOperation.SourceTexture->GetDesc().Format, Ptr, MipWidth, MipHeight, Width, CurrentMipIndex, CurrentSliceIndex))
					{
						SetTestStageFailure(ETestStage_Clear, CurrentMipIndex, CurrentSliceIndex);
						return false;
					}
					return true;
				}
			);

			if (bTestSuccess)
			{
				++NumClearSuccess;
			}
		}

		return NumClearSuccess;
	}

	void DoWork()
	{
		CLEAR_TEST_SCOPED_NAMED_EVENT_F(TEXT("TextureTestTask %s"), FColor::Magenta, *GetTextureName(Desc));
		uint32 NumClearSuccess =0;
		if (ProcessDrawResults())
		{
			NumClearSuccess = ProcessClearResults();
		}

		if (bReportResultsOnEnd)
		{
			TaskTestContext->SetClearTestResult(TaskTextureIndex, NumClearSuccess, TestStageFailure);
		}

		OnWorkDone();
	}

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		DoWork();
	}

	void OnWorkDone()
	{
		TaskTestContext->StagingTexturePool.ReturnToPool(TestOperations);
	}

};

static FGraphEventRef CreateTextureTestTasks(const TArrayView<FTestOperation>& TestOperations, FTestContext& TestContext, const FRHITextureDesc& Desc, bool bInReportResultsOnEnd)
{
	if (GSerialTextureTestTask)
	{
		FTextureTestTasks TextureTestTasks(TestOperations, TestContext, Desc, bInReportResultsOnEnd);
		TextureTestTasks.DoWork();
		FGraphEventRef GraphEventRef = FGraphEvent::CreateGraphEvent();
		GraphEventRef->DispatchSubsequents();
		return GraphEventRef;
	}
	else
	{
		return TGraphTask<FTextureTestTasks>::CreateTask().ConstructAndDispatchWhenReady(TestOperations, TestContext, Desc, bInReportResultsOnEnd);
	}
}

static TArray<FGraphEventRef> DispatchTestValidation(const TArrayView<FTestOperation>& TestOperations, FTestContext& TestContext, const FRHITextureDesc& Desc, bool bInReportResultsOnEnd)
{
	TArray<FGraphEventRef> AllGraphEvents;
	if (Desc.ArraySize * Desc.Depth > 1)
	{
		TArrayView<FTestOperation> TestOperationsCopy = TestOperations;
		while (!TestOperationsCopy.IsEmpty())
		{
			AllGraphEvents.Add(CreateTextureTestTasks(TestOperationsCopy.Left(1), TestContext, Desc, bInReportResultsOnEnd));
			TestOperationsCopy = TestOperationsCopy.RightChop(1);
		}

	}
	else
	{
		AllGraphEvents.Add(CreateTextureTestTasks(TestOperations, TestContext, Desc, bInReportResultsOnEnd));
	}
	return AllGraphEvents;
}

static void WaitOnDispatchTestValidation(FRHICommandListImmediate& RHICmdList, TArray<FGraphEventRef>& AllGraphEvents, int32 MaxTasksToWait = INT32_MAX)
{
	int32 NumGraphEventsToWait = FMath::Min(MaxTasksToWait, AllGraphEvents.Num());
	CLEAR_TEST_SCOPED_NAMED_EVENT_F(TEXT("WaitForTaskToFinish %d / %d"), FColor::Magenta, NumGraphEventsToWait, AllGraphEvents.Num());

	int32 GraphEventIndex = 0;
	for (; GraphEventIndex < NumGraphEventsToWait; ++GraphEventIndex)
	{
		FGraphEventRef& GraphEvent = AllGraphEvents[GraphEventIndex];
		GraphEvent->Wait();
	}

	if (NumGraphEventsToWait < AllGraphEvents.Num())
	{
		TArray<FGraphEventRef> NewGraphArray;
		NewGraphArray.Reserve(AllGraphEvents.Num() - NumGraphEventsToWait);
		for (; GraphEventIndex < AllGraphEvents.Num(); ++GraphEventIndex)
		{
			NewGraphArray.Add(AllGraphEvents[GraphEventIndex]);
		}

		AllGraphEvents = NewGraphArray;
	}
	else
	{
		AllGraphEvents.Empty();
	}
}

static void BlockGPUAndLaunchTestTasks(FRHICommandListImmediate& RHICmdList, const TArrayView<FTestOperation>& TestOperationGroup, FTestContext& TestContext, FRHITexture* SourceTexture)
{
	{
		// Also flushes the RHI thread
		CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT("Test_ClearTexture_BlockUntilGPUIdle", FColor::Magenta);
		RHICmdList.SubmitAndBlockUntilGPUIdle();
	}

	for (FTestOperation& TestOperation : TestOperationGroup)
	{
		CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT("Test_ClearTexture_MapStagingSurfaces", FColor::Magenta);
		MapStagingSurfaces(RHICmdList, TestOperation.DrawResult);
		MapStagingSurfaces(RHICmdList, TestOperation.ClearResult);
	}

	TestContext.AllEvents.Append(DispatchTestValidation(TestOperationGroup, TestContext, SourceTexture->GetDesc(), true));
}

static void ProcessTestOperations(FRHICommandListImmediate& RHICmdList, FTestBatchRequestIteration& TestBatchRequestIteration, FRHITexture* SourceTexture, FRHIShaderResourceView* SourceSRV, FTestContext& TestContext)
{
	CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT(TEXT("ProcessTestOperations"), FColor::Magenta);

	// Each entry in a TestBatchRequest contains the different textures we want to test clear for a given slice/mip combination. 
	// This allows us to not have to wait for the GPU between every test
	FStagingTexturePool& StagingTexturePool = TestContext.StagingTexturePool;
	const FVertexDeclarationRHIRef& VertexDeclarationRHI = TestContext.VertexDeclarationRHI;
	const FVector4f& SentinelColor = TestContext.SentinelColor;

	{
		CLEAR_TEST_SCOPED_NAMED_EVENT_F(TEXT("%d Clears"), FColor::Magenta, TestBatchRequestIteration.TestOperations.Num());
		int32 TestOperationGroupStart = 0;
		int32 TestOperationGroupEnd = 0;
		int32 TextureAllocated = 0;

		for (FTestOperation& TestOperation : TestBatchRequestIteration.TestOperations)
		{
			CLEAR_TEST_SCOPED_DRAW_EVENTF(RHICmdList, ClearTextureTest, TEXT("Test mip %d slice %d"), TestOperation.TestMipIndex, TestOperation.TestArrayIndex);

			FUnorderedAccessViewRHIRef TmpTexture2DAtlasUAV;
			FTextureRHIRef TmpTexture2DAtlas = StagingTexturePool.CreateTexture2DAtlasRTFromPool(RHICmdList, TestOperation.SourceTexture->GetDesc().Format, &TmpTexture2DAtlasUAV);
			// Perform the clear operation itself on all the textures of the batch
			FTextureRHIRef Texture = TestOperation.SourceTexture;
			int32 TestArrayIndex = TestOperation.TestArrayIndex;
			int32 TestMipIndex = TestOperation.TestMipIndex;
			check(SourceTexture == Texture);

			{
				CLEAR_TEST_SCOPED_DRAW_EVENT(RHICmdList, Verify_sentinel);
				TestOperation.DrawResult.StagingTexture = StagingTexturePool.CreateTexture2DAtlasStagingFromPool(RHICmdList, TestOperation.SourceTexture->GetDesc().Format);
				++TextureAllocated;
				CopyTextureToUnpackedStaging(RHICmdList, SourceTexture, SourceSRV, TmpTexture2DAtlas, TmpTexture2DAtlasUAV, TestOperation.DrawResult.StagingTexture, VertexDeclarationRHI);
				RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::SRVCompute, ERHIAccess::RTV));
			}
			{
				CLEAR_TEST_SCOPED_DRAW_EVENT(RHICmdList, Clear_and_Verify_pattern);
				TestOperation.ClearResult.StagingTexture = StagingTexturePool.CreateTexture2DAtlasStagingFromPool(RHICmdList, TestOperation.SourceTexture->GetDesc().Format);
				++TextureAllocated;
				FRHIRenderPassInfo RenderPassInfo(Texture.GetReference(), ERenderTargetActions::Clear_Store, nullptr, TestMipIndex, TestArrayIndex);
				RHICmdList.BeginRenderPass(RenderPassInfo, TEXT("Test_Clear_DrawColoredQuad"));
				RHICmdList.EndRenderPass();

				RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::RTV, ERHIAccess::SRVCompute));
			
				CopyTextureToUnpackedStaging(RHICmdList, SourceTexture, SourceSRV, TmpTexture2DAtlas, TmpTexture2DAtlasUAV, TestOperation.ClearResult.StagingTexture, VertexDeclarationRHI);
			}
			{
				CLEAR_TEST_SCOPED_DRAW_EVENT(RHICmdList, Draw_mip_to_sentinel);
				RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::SRVCompute, ERHIAccess::RTV));
				DrawColoredQuad(RHICmdList, VertexDeclarationRHI, Texture, SentinelColor, TestMipIndex, TestArrayIndex);
				RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::RTV, ERHIAccess::SRVCompute));
			}

			++TestOperationGroupEnd;

			// Kick tasks to ensure next iteration won't wait forever
			if (TextureAllocated + 2 > GMaxStagingTexturesToAllocate)
			{
				TArrayView<FTestOperation> TestOperationGroup = MakeArrayView(&TestBatchRequestIteration.TestOperations[TestOperationGroupStart], TestOperationGroupEnd - TestOperationGroupStart);
				BlockGPUAndLaunchTestTasks(RHICmdList, TestOperationGroup, TestContext, SourceTexture);
				TestOperationGroupStart = TestOperationGroupEnd;
				TextureAllocated = 0;
			}
		}

		if (TestOperationGroupEnd > TestOperationGroupStart)
		{
			TArrayView<FTestOperation> TestOperationGroup = MakeArrayView(&TestBatchRequestIteration.TestOperations[TestOperationGroupStart], TestOperationGroupEnd - TestOperationGroupStart);
			BlockGPUAndLaunchTestTasks(RHICmdList, TestOperationGroup, TestContext, SourceTexture);
		}
	}

}

static void FillClearAndTest(FRHICommandListImmediate& RHICmdList, FRHITexture* SourceTexture, FRHIShaderResourceView* SourceSRV, FTestContext& TestContext)
{
	// Optim - We generate a lot of RHI commands for slices + mips: Draw once to sentinel color everywhere, make sure everything has been written properly, then clear + draw the same region
	{
		CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT(TEXT("Fill_Validate"), FColor::Magenta);

		FTestOperation TestOperation = FTestOperation{ SourceTexture, FStagingData(), FStagingData(), -1, -1 };
		{
			FUnorderedAccessViewRHIRef TmpTexture2DAtlasUAV;
			FTextureRHIRef TmpTexture2DAtlas = TestContext.StagingTexturePool.CreateTexture2DAtlasRTFromPool(RHICmdList, SourceTexture->GetDesc().Format, &TmpTexture2DAtlasUAV);
			TestOperation.DrawResult.StagingTexture = TestContext.StagingTexturePool.CreateTexture2DAtlasStagingFromPool(RHICmdList, TestOperation.SourceTexture->GetDesc().Format);

			{
				CLEAR_TEST_SCOPED_DRAW_EVENT(RHICmdList, Fill_whole_texture_to_sentinel);
				RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::SRVCompute, ERHIAccess::RTV));
				DrawColoredQuad(RHICmdList, TestContext.VertexDeclarationRHI, SourceTexture, TestContext.SentinelColor);
			}

			{
				CLEAR_TEST_SCOPED_DRAW_EVENT(RHICmdList, Verify_sentinel);
				RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::RTV, ERHIAccess::SRVCompute));
				CopyTextureToUnpackedStaging(RHICmdList, SourceTexture, SourceSRV, TmpTexture2DAtlas, TmpTexture2DAtlasUAV, TestOperation.DrawResult.StagingTexture, TestContext.VertexDeclarationRHI);
			}
		}

		RHICmdList.SubmitAndBlockUntilGPUIdle();

		MapStagingSurfaces(RHICmdList, TestOperation.DrawResult);

		TArray<FTestOperation> TestOperations;
		TestOperations.Add(TestOperation);
		TArray<FGraphEventRef> AllEvents = DispatchTestValidation(TestOperations, TestContext, SourceTexture->GetDesc(), false);
		WaitOnDispatchTestValidation(RHICmdList, AllEvents);
	}

	FTestBatchRequestIteration TestBatchRequestIteration = {};
	uint32 ExpectedNumClears = TestBatchRequestIteration.CollectTextureOperations(SourceTexture);
	TestContext.SetExpectedNumClears(0, ExpectedNumClears);

	ProcessTestOperations(RHICmdList, TestBatchRequestIteration, SourceTexture, SourceSRV, TestContext);

}

static void WriteTestData(void* Ptr, int32 Width, int32 Height, int32 Stride, uint32 MipIndex, uint32 SliceIndex, EPixelFormat PixelFormat)
{
	const float ColorMultiplier = (PixelFormat == PF_R8G8B8A8) ? 1.0f / 255.0f : 1.0f;
	const uint32 BytesPerPixel = GPixelFormats[PixelFormat].BlockBytes;;

	uint8* Row = (uint8*)Ptr;
	for (int32 Y = 0; Y < Height; ++Y)
	{
		uint8* Pixel = Row;
		for (int32 X = 0; X < Width; ++X)
		{
			uint8 TextureColor[16];
			FLinearColor LinearColor(X * ColorMultiplier, Y * ColorMultiplier, MipIndex * ColorMultiplier, SliceIndex * ColorMultiplier);
			ConvertLinearColorToTextureColor(TextureColor, PixelFormat, LinearColor);
			FMemory::Memcpy(Pixel, TextureColor, BytesPerPixel);
			Pixel += BytesPerPixel;
		}

		Row += Stride;
	}
}

static bool CheckTestData(const FMappedPixel& Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 MipIndex, uint32 SliceIndex, EPixelFormat PixelFormat)
{
	const float ColorMultiplier = (PixelFormat == PF_R8G8B8A8) ? 1.0f / 255.0f : 1.0f;
	FMappedPixel Row = Ptr;

	EPixelFormat AtlasPixelFormat = PixelFormat;
	uint32 MappedBytesPerPixel = GPixelFormats[AtlasPixelFormat].BlockBytes;

	for (uint32 Y = 0; Y < MipHeight; ++Y)
	{
		FMappedPixel Pixel = Row;
		for (uint32 X = 0; X < MipWidth; ++X)
		{
			uint8 TextureColor[16];
			FLinearColor LinearColor(X * ColorMultiplier, Y * ColorMultiplier, MipIndex * ColorMultiplier, SliceIndex * ColorMultiplier);
			ConvertLinearColorToTextureColor(TextureColor, PixelFormat, LinearColor);
			if (!ComparePixelValues(TextureColor, Pixel, PixelFormat))
			{
				return false;
			}
			Pixel.MoveForward(MappedBytesPerPixel);
		}
		Row.MoveForward(Width * MappedBytesPerPixel);
	}
	return true;
}

static FShaderResourceViewRHIRef CreateClearTextureSRV(FRHICommandListImmediate& RHICmdList, FRHITexture* Texture)
{
	if (!GUseCustomSRV)
	{
		return nullptr;
	}
	const FRHITextureDesc& Desc = Texture->GetDesc();
	int32 NumSlices = ComputeNumSlices(Desc, 0);
	ETextureDimension SRVDimension = Desc.IsTextureCube() ? ETextureDimension::Texture2DArray : Desc.Dimension;
	FRHIViewDesc::FTextureSRV::FInitializer CreateDesc = FRHIViewDesc::CreateTextureSRV().SetDimension(SRVDimension)
		.SetFormat(Desc.Format)
		.SetMipRange(0, Desc.NumMips)
		.SetArrayRange(0, Desc.IsTexture3D() ? 1 : NumSlices)
		.SetPlane(ERHITexturePlane::Primary)
		;

	return RHICmdList.CreateShaderResourceView(Texture, CreateDesc);

}


static bool TestDrawAtlasing(FRHICommandListImmediate& RHICmdList, FVertexDeclarationRHIRef VertexDeclarationRHI, const FRHITextureCreateDesc& RHITextureCreateDesc)
{
	FTextureRHIRef SourceTexture = RHICmdList.CreateTexture(RHITextureCreateDesc);

	// Create the SourceTexture
	for (uint32 MipIndex=0; MipIndex < RHITextureCreateDesc.NumMips; ++MipIndex)
	{
		int32 NumSlices = ComputeNumSlices(RHITextureCreateDesc, MipIndex);
		for (int32 SliceIndex=0; SliceIndex < NumSlices; ++SliceIndex)
		{
			void* Data;
			uint32 Stride;
			uint32 MipWidth = FMath::Max(RHITextureCreateDesc.Extent.X >> MipIndex, 1);
			uint32 MipHeight = FMath::Max(RHITextureCreateDesc.Extent.Y >> MipIndex, 1);
			
			switch (RHITextureCreateDesc.Dimension)
			{
			case ETextureDimension::Texture2D:
				Data = RHICmdList.LockTexture2D(SourceTexture, MipIndex, RLM_WriteOnly, Stride, false);
				WriteTestData(Data, MipWidth, MipHeight, Stride, MipIndex, SliceIndex, RHITextureCreateDesc.Format);
				RHICmdList.UnlockTexture2D(SourceTexture, MipIndex, false);
				break;
			case ETextureDimension::TextureCubeArray:
				Data = RHICmdList.LockTextureCubeFace(SourceTexture, SliceIndex%6, SliceIndex/6, MipIndex, RLM_WriteOnly, Stride, false);
				WriteTestData(Data, MipWidth, MipHeight, Stride, MipIndex, SliceIndex, RHITextureCreateDesc.Format);
				RHICmdList.UnlockTextureCubeFace(SourceTexture, SliceIndex % 6, SliceIndex / 6, MipIndex, false);
				break;
			case ETextureDimension::TextureCube:
				Data = RHICmdList.LockTextureCubeFace(SourceTexture, SliceIndex, 0, MipIndex, RLM_WriteOnly, Stride, false);
				WriteTestData(Data, MipWidth, MipHeight, Stride, MipIndex, SliceIndex, RHITextureCreateDesc.Format);
				RHICmdList.UnlockTextureCubeFace(SourceTexture, SliceIndex, 0, MipIndex, false);
				break;
			case ETextureDimension::Texture2DArray:
				Data = RHICmdList.LockTexture2DArray(SourceTexture, SliceIndex, MipIndex, RLM_WriteOnly, Stride, false);
				WriteTestData(Data, MipWidth, MipHeight, Stride, MipIndex, SliceIndex, RHITextureCreateDesc.Format);
				RHICmdList.UnlockTexture2DArray(SourceTexture, SliceIndex, MipIndex, false);
				break;
			case ETextureDimension::Texture3D:
			{
				FUpdateTextureRegion3D UpdateTextureRegion3D(FIntVector(0, 0, SliceIndex), FIntVector::ZeroValue, FIntVector(MipWidth, MipHeight, 1));
				FUpdateTexture3DData UpdateTexture3DData = RHICmdList.BeginUpdateTexture3D(SourceTexture, MipIndex, UpdateTextureRegion3D);
				WriteTestData(UpdateTexture3DData.Data, MipWidth, MipHeight, UpdateTexture3DData.RowPitch, MipIndex, SliceIndex, RHITextureCreateDesc.Format);
				RHICmdList.EndUpdateTexture3D(UpdateTexture3DData);
			}
				break;
			default:
				checkNoEntry();
				break;
			}
		}
	}

	FShaderResourceViewRHIRef SourceTextureSRV = CreateClearTextureSRV(RHICmdList, SourceTexture);

	FUnorderedAccessViewRHIRef TmpTexture2DAtlasUAV;
	FTextureRHIRef TmpTexture2DAtlas = CreateTexture2DAtlasRT(RHICmdList, SourceTexture->GetDesc().Format, &TmpTexture2DAtlasUAV);
	FTextureRHIRef DestTextureStaging = CreateTexture2DAtlasStaging(RHICmdList, SourceTexture->GetDesc().Format);

	RHICmdList.Transition(FRHITransitionInfo(DestTextureStaging, ERHIAccess::CPURead, ERHIAccess::CopyDest));
	RHICmdList.Transition(FRHITransitionInfo(SourceTexture, ERHIAccess::CopySrc, ERHIAccess::SRVCompute));
	CopyTextureToUnpackedStaging(RHICmdList, SourceTexture, SourceTextureSRV, TmpTexture2DAtlas, TmpTexture2DAtlasUAV, DestTextureStaging, VertexDeclarationRHI);

	RHICmdList.SubmitAndBlockUntilGPUIdle();
	FStagingData DrawResult{ DestTextureStaging };
	MapStagingSurfaces(RHICmdList, DrawResult);
	bool bTestSuccess = VerifyTextureData(DrawResult, SourceTexture, CheckTestData);
	UnmapStagingSurfaces(RHICmdList, DrawResult);
	return bTestSuccess;
}

bool FRHIClearTextureTests::Test_ClearTexture(FRHICommandListImmediate& RHICmdList)
{
	CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT("Test_ClearTexture", FColor::Magenta);

	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

	FVertexDeclarationElementList VertexDeclarationElements;
	FVertexDeclarationRHIRef VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(VertexDeclarationElements);

	const TArray<EPixelFormat> PixelFormatsArray = { PF_R8G8B8A8, PF_FloatRGBA, PF_A32B32G32R32F };
	const TArray<uint32> NumMipsArray = { 1, 0 };
	const TArray<ETextureDimension> TextureDimensionArray = { ETextureDimension::Texture2D, ETextureDimension::Texture2DArray, ETextureDimension::Texture3D, ETextureDimension::TextureCube, ETextureDimension::TextureCubeArray };
	const TArray<ETextureCreateFlags> TextureCreateFlagsArray = { TexCreate_None, TexCreate_DisableDCC, TexCreate_NoFastClear };
	const TArray<int32> ExtentsArray = { 16, 128 };
	const TArray<int32> DepthOrArraySizeArray = { 1, 0 }; // Every dimension will choose its own depth/array size
	TArray<FVector4f> ClearColorLinearArray =
	{
  		FVector4f(0.0f, 0.0f, 0.0f, 0.0f) ,
  		FVector4f(0.0f, 0.0f, 0.0f, 1.0f) ,
  		FVector4f(1.0f, 1.0f, 1.0f, 0.0f) ,
  		FVector4f(1.0f, 1.0f, 1.0f, 1.0f) ,
 		FVector4f(0.2345f, 0.8499f, 0.00145f, 0.417f)
	};

// 	const TArray<EPixelFormat> PixelFormatsArray = { PF_R8G8B8A8 };
// 	const TArray<uint32> NumMipsArray = { 1 };
// 	const TArray<ETextureDimension> TextureDimensionArray = { ETextureDimension::Texture2DArray };
// 	const TArray<ETextureCreateFlags> TextureCreateFlagsArray = { TexCreate_None };
// 	const TArray<int32> ExtentsArray = { 16 };

	bool bTestDrawAtlasing = true;

	if (GValidateDrawAtlasing)
	{
		// Check that the atlasing works, R=coordX, G=coordY, B=mipindex, A=sliceindex
		for (EPixelFormat PixelFormat : PixelFormatsArray)
		{
			bTestDrawAtlasing &= TestDrawAtlasing(RHICmdList, VertexDeclarationRHI,
				FRHITextureCreateDesc::Create2D(TEXT("SourceTexture"), 128, 128, PixelFormat).SetNumMips(8).SetInitialState(ERHIAccess::CopySrc));

			bTestDrawAtlasing &= TestDrawAtlasing(RHICmdList, VertexDeclarationRHI,
				FRHITextureCreateDesc::CreateCube(TEXT("SourceTexture"), 32, PixelFormat).SetNumMips(6).SetInitialState(ERHIAccess::CopySrc));

			bTestDrawAtlasing &= TestDrawAtlasing(RHICmdList, VertexDeclarationRHI,
				FRHITextureCreateDesc::CreateCubeArray(TEXT("SourceTexture"), 32, 3, PixelFormat).SetNumMips(6).SetInitialState(ERHIAccess::CopySrc));

			bTestDrawAtlasing &= TestDrawAtlasing(RHICmdList, VertexDeclarationRHI,
				FRHITextureCreateDesc::Create2DArray(TEXT("SourceTexture"), 32, 32, 5, PixelFormat).SetNumMips(6).SetInitialState(ERHIAccess::CopySrc));

			bTestDrawAtlasing &= TestDrawAtlasing(RHICmdList, VertexDeclarationRHI,
				FRHITextureCreateDesc::Create3D(TEXT("SourceTexture"), 32, 32, 5, PixelFormat).SetNumMips(1).SetInitialState(ERHIAccess::CopySrc));

			RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
		}
	}


	if (!bTestDrawAtlasing)
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Test_ClearTexture\" 2D atlasing failed, aborting");
		return false;
	}

	bool bSupportsSM5 = IsFeatureLevelSupported(GMaxRHIShaderPlatform, ERHIFeatureLevel::SM5);

	// Prune not only invalid combinations, but also try to keep the number of clear as low as possible to keep testing time acceptable
	auto AddValidTextureDescIfValid = [bSupportsSM5](const FRHITextureCreateDesc& InRHITextureCreateDesc, TArray<FRHITextureCreateDesc>& OutRHITextureCreateDescs)
	{
		FRHITextureCreateDesc RHITextureCreateDesc = InRHITextureCreateDesc;
		if (RHITextureCreateDesc.Dimension == ETextureDimension::Texture2D)
		{
		}

		if (RHITextureCreateDesc.Dimension == ETextureDimension::Texture3D)
		{
			if (!bSupportsSM5) return; // non sm5 platforms do not support write to multiple slices and TexCreate_TargetArraySlicesIndependently is not supported for volume textures
			if (RHITextureCreateDesc.Extent.X > 16) return;
			if (RHITextureCreateDesc.NumMips > 1) return;
			RHITextureCreateDesc.Depth = 3;
		}

		if (RHITextureCreateDesc.Dimension == ETextureDimension::TextureCube)
		{
			if (RHITextureCreateDesc.Extent.X > 16) return;
			if (RHITextureCreateDesc.Depth > 1) return;

			// only support write to individual slices
			RHITextureCreateDesc.Flags |= TexCreate_TargetArraySlicesIndependently;
		}

		if (RHITextureCreateDesc.Dimension == ETextureDimension::TextureCubeArray)
		{
			if (RHITextureCreateDesc.Extent.X > 16) return;
			if (RHITextureCreateDesc.Depth > 1) return;

			RHITextureCreateDesc.ArraySize = 3;
			// only support write to individual slices
			RHITextureCreateDesc.Flags |= TexCreate_TargetArraySlicesIndependently;
		}


		if (RHITextureCreateDesc.Dimension == ETextureDimension::Texture2DArray)
		{
			if (RHITextureCreateDesc.Extent.X > 16) return;
			if (RHITextureCreateDesc.Depth > 1) return;

			RHITextureCreateDesc.ArraySize = 4;
			// ignore SM5 support here since we can clear to individual slices
			{
				FRHITextureCreateDesc RHITextureCreateDescCopy = RHITextureCreateDesc;
				RHITextureCreateDescCopy.Flags |= TexCreate_TargetArraySlicesIndependently;
				OutRHITextureCreateDescs.Add(RHITextureCreateDescCopy);
			}
			
			if (!bSupportsSM5) return; // non sm5 platforms do not support write to multiple slices
		}

		check(RHITextureCreateDesc.Depth == 1 || RHITextureCreateDesc.ArraySize == 1);
		check(RHITextureCreateDesc.Depth > 0 && RHITextureCreateDesc.ArraySize > 0);

		OutRHITextureCreateDescs.Add(RHITextureCreateDesc);

	};

	FVector4f SentinelColor = FVector4f(0.1234f, 0.5678f, 0.9012f, 0.3456f);

	TArray<FRHITextureCreateDesc> RHITextureCreateDescs;
	RHITextureCreateDescs.Reserve(1000);

	for (EPixelFormat PixelFormat : PixelFormatsArray)
	for (ETextureCreateFlags InTextureCreateFlags : TextureCreateFlagsArray)
	for (FVector4f ClearColorLinear : ClearColorLinearArray)
	for (uint32 DepthOrArraySize : DepthOrArraySizeArray)
	for (uint32 InNumMips : NumMipsArray)
	for (int32 Extent : ExtentsArray)
	for (ETextureDimension TextureDimension : TextureDimensionArray)
	{
		ETextureCreateFlags TextureCreateFlags = TexCreate_ShaderResource | TexCreate_RenderTargetable | InTextureCreateFlags;
		uint32 NumMips = (InNumMips == 0) ? 1 + FMath::FloorLog2(Extent) : InNumMips;
		int32 ArraySize = (TextureDimension == ETextureDimension::Texture3D) ? 1 : DepthOrArraySize;
		int32 DepthExtent = (TextureDimension == ETextureDimension::Texture3D) ? DepthOrArraySize : 1;
		FRHITextureCreateDesc Desc = FRHITextureCreateDesc(TEXT("TestClearTexture"), TextureDimension)
			.SetFormat(PixelFormat)
			.SetExtent(Extent, Extent)
			.SetDepth(DepthExtent)
			.SetNumMips(NumMips)
			.SetArraySize(ArraySize)
			.SetFlags(TextureCreateFlags)
			.SetClearValue(FClearValueBinding(ClearColorLinear))
			.SetInitialState(ERHIAccess::SRVCompute);

		bool bIsForArray = (DepthOrArraySize == 0);
		bool bIsArray = Desc.IsTextureArray() || Desc.IsTexture3D();

		if (bIsForArray != bIsArray)
		{
			continue;
		}

		AddValidTextureDescIfValid(Desc, RHITextureCreateDescs);
	}
	
	bool bResult = true;
	{
		FTestContext TestContext(VertexDeclarationRHI, RHITextureCreateDescs, SentinelColor);
		EPixelFormat CurrentPixelFormat = PF_Unknown;

		bool bGGPUCaptureTest = GGPUCaptureTest;
		bool bGPUCapture = bGGPUCaptureTest && IRenderCaptureProvider::IsAvailable();
		if (bGPUCapture)
		{
			Test_ClearTextureBeginCapture(RHICmdList);
		}

		for (int32 TextureIndex = 0; TextureIndex < RHITextureCreateDescs.Num(); ++TextureIndex)
		{
			bool bFormatChanged = CurrentPixelFormat != RHITextureCreateDescs[TextureIndex].Format;

			// Do not keep staging textures in memory for a given pixel format
			if (bFormatChanged)
			{
				CurrentPixelFormat = RHITextureCreateDescs[TextureIndex].Format;

				WaitOnDispatchTestValidation(RHICmdList, TestContext.AllEvents);
				{
					CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT(TEXT("FlushStagingTextures"), FColor::Magenta);
					TestContext.StagingTexturePool.FlushStagingTextures(RHICmdList, CurrentPixelFormat);
					TestContext.StagingTexturePool.PreallocateStagingTexture(RHICmdList, CurrentPixelFormat, GMaxStagingTexturesToAllocate);
				}
			}

			{
				CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT(TEXT("FlushResources"), FColor::Magenta);
				RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
			}

			CLEAR_TEST_SCOPED_NAMED_EVENT_F(TEXT("Test_ClearTexture %s"), FColor::Magenta, *GetTextureName(RHITextureCreateDescs[TextureIndex]));
			CLEAR_TEST_SCOPED_DRAW_EVENTF(RHICmdList, ClearTextureTest, TEXT("Test_ClearTexture %s"), GetTextureName(RHITextureCreateDescs[TextureIndex]));

			FTextureRHIRef Texture;
			FShaderResourceViewRHIRef TextureSRV;
			{
				CLEAR_TEST_SCOPED_NAMED_EVENT_TEXT(TEXT("CreateTexture"), FColor::Magenta);
				Texture = RHICmdList.CreateTexture(RHITextureCreateDescs[TextureIndex]);
				TextureSRV = CreateClearTextureSRV(RHICmdList, Texture);
			}
			TestContext.BatchStartTextureIndex = TextureIndex;
			FillClearAndTest(RHICmdList, Texture, TextureSRV, TestContext);

		}
		WaitOnDispatchTestValidation(RHICmdList, TestContext.AllEvents);
		TestContext.StagingTexturePool.FlushAllStagingTextures(RHICmdList);

		if (bGPUCapture)
		{
			Test_ClearTextureEndCapture(RHICmdList);
		}

		uint32 NumClearsSuccess = 0;
		uint32 NumClearsDone = 0;
		for (int32 TextureIndex = 0; TextureIndex < TestContext.RHITextureCreateDescs.Num(); ++TextureIndex)
		{
			uint32 TextureNumClears = TestContext.TextureNumClears[TextureIndex];
			uint32 TextureNumSuccess = TestContext.TextureNumsClearsSuccess[TextureIndex];
			NumClearsDone += TextureNumClears;
			NumClearsSuccess += TextureNumSuccess;
			if (TextureNumClears != TextureNumSuccess)
			{
				const FTestStageFailure& TestStageFailure = TestContext.TestStageFailures[TextureIndex];
				const FString FailReason = FString::Printf(TEXT("Stage %d, Mip %d, Slice %d"), TestStageFailure.TestStage, TestStageFailure.MipIndex, TestStageFailure.SliceIndex);
				UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Test_ClearTexture for %ls. %ls (%d/%d clear tests)\"", 
					*GetTextureName(TestContext.RHITextureCreateDescs[TextureIndex]), *FailReason, TextureNumSuccess, TextureNumClears);
			}
		}
		bResult = NumClearsSuccess == NumClearsDone;

		if (bResult)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"Test_ClearTexture (%d/%d clear tests)\"", NumClearsSuccess, NumClearsDone);
		}
		else
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Test_ClearTexture (%d/%d clear tests)\"", NumClearsSuccess, NumClearsDone);
		}

		if (GBreakOnTestEnd)
		{
			GLog->Flush();
			UE_DEBUG_BREAK();
		}

	}

	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

	return bResult;
}
