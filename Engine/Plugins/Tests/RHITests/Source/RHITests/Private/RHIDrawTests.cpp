// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIDrawTests.h"
#include "RHIBufferTests.h" // for VerifyBufferContents
#include "CommonRenderResources.h"
#include "RenderCaptureInterface.h"
#include "RHIStaticStates.h"
#include "ShaderCompilerCore.h"

class FTestDrawInstancedVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTestDrawInstancedVS);
	FTestDrawInstancedVS() = default;
	FTestDrawInstancedVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_IndirectDraw);
	}
};
IMPLEMENT_GLOBAL_SHADER(FTestDrawInstancedVS, "/Plugin/RHITests/Private/TestDrawInstanced.usf", "TestDrawInstancedMainVS", SF_Vertex);

class FTestDrawInstancedPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTestDrawInstancedPS);
	FTestDrawInstancedPS() = default;
	FTestDrawInstancedPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		OutDrawnInstances.Bind(Initializer.ParameterMap, TEXT("OutDrawnInstances"), SPF_Mandatory);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}
	LAYOUT_FIELD(FShaderResourceParameter, OutDrawnInstances);
};
IMPLEMENT_GLOBAL_SHADER(FTestDrawInstancedPS, "/Plugin/RHITests/Private/TestDrawInstanced.usf", "TestDrawInstancedMainPS", SF_Pixel);

namespace
{

// Structure to initialize common resources required for various draw tests
struct FDrawTestResources
{
	FDrawTestResources(FRHICommandListImmediate& RHICmdList)
		: VertexShader(GetGlobalShaderMap(GMaxRHIFeatureLevel))
		, PixelShader(GetGlobalShaderMap(GMaxRHIFeatureLevel))
	{		
		FRHITextureDesc RenderTargetTextureDesc(ETextureDimension::Texture2D, ETextureCreateFlags::RenderTargetable, PF_B8G8R8A8, FClearValueBinding(), RenderTargetSize, 1, 1, 1, 1, 0);
		FRHITextureCreateDesc RenderTargetCreateDesc(RenderTargetTextureDesc, ERHIAccess::RTV, TEXT("DrawTest_RenderTarget"));
		RenderTarget = RHICmdList.CreateTexture(RenderTargetCreateDesc);

		FVertexDeclarationElementList VertexDeclarationElements;
		// Position vertex element
		VertexDeclarationElements.Add(FVertexElement(0, 0, VET_Float4, 0, 16, false /*per instance frequency*/));
		// Vertex element to access instance ID buffer (uint per instance)
		VertexDeclarationElements.Add(FVertexElement(1, 0, VET_UInt, 1, 4, true /*per instance frequency*/));

		VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(VertexDeclarationElements);

		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = VertexDeclarationRHI;
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.PrimitiveType = EPrimitiveType::PT_TriangleList;

		// D3D12 does not have a way to get the base instance ID (SV_InstanceID always starts from 0), so we must emulate it...
		uint32 InstanceIDs[MaxInstances];
		for (uint32 i = 0; i < MaxInstances; ++i)
		{
			InstanceIDs[i] = i;
		}
		InstanceIDBuffer = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("DrawTest_InstanceID"), MakeConstArrayView(InstanceIDs));

		// Indices for 4 triangles
		const uint16 Indices[NumTotalVertices] =
		{
			0, 1, 2, // valid
			3, 4, 5, // valid
			6, 7, 8, // valid
			9, 10, 11 // degenerate
		};

		IndexBuffer = UE::RHIResourceUtils::CreateIndexBufferFromArray(RHICmdList, TEXT("DrawTest_IndexBuffer"), MakeConstArrayView(Indices));

		TArray<FVector4f> Vertices;
		Vertices.Reserve(NumTotalVertices);
		for (uint32 i = 0; i < NumValidVertices; ++i)
		{
			switch (i % 3)
			{
			case 0:	Vertices.Add(FVector4f(-1.0f, -1.0f, 0.0f, 1.0f)); break;
			case 1:	Vertices.Add(FVector4f(-1.0f, +3.0f, 0.0f, 1.0f)); break;
			case 2:	Vertices.Add(FVector4f(+3.0f, -1.0f, 0.0f, 1.0f)); break;
			}
		}
		for (uint32 i = NumValidVertices; i < NumTotalVertices; ++i)
		{
			Vertices.Add(FVector4f(0.0f, 0.0f, 0.0f, 1.0f));
		}
		VertexBuffer = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("DrawTest_VertexBuffer"), MakeConstArrayView(Vertices));

		static constexpr uint32 OutputBufferStride = sizeof(uint32);
		static constexpr uint32 OutputBufferSize = OutputBufferStride * MaxInstances;
		constexpr EBufferUsageFlags OutputBufferUsage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy;

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("DrawTest_OutputBuffer"), OutputBufferSize, OutputBufferStride, OutputBufferUsage)
			.SetInitialState(ERHIAccess::UAVCompute);
		OutputBuffer = RHICmdList.CreateBuffer(CreateDesc);

		OutputBufferUAV = RHICmdList.CreateUnorderedAccessView(OutputBuffer,
			FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_UINT));
	}

	static constexpr uint32 MaxInstances = 8;

	FVertexDeclarationRHIRef VertexDeclarationRHI;
	TShaderMapRef<FTestDrawInstancedVS> VertexShader;
	TShaderMapRef<FTestDrawInstancedPS> PixelShader;
	FGraphicsPipelineStateInitializer GraphicsPSOInit;

	FIntPoint RenderTargetSize = FIntPoint(4, 4);
	FTextureRHIRef RenderTarget;

	FBufferRHIRef InstanceIDBuffer;
	FBufferRHIRef IndexBuffer;

	// 3 full screen triangles, followed by a degenerate triangle
	static constexpr uint32 NumTotalVertices = 12;
	static constexpr uint32 NumValidVertices = 9;
	FBufferRHIRef VertexBuffer;

	FBufferRHIRef OutputBuffer;

	FUnorderedAccessViewRHIRef OutputBufferUAV;
};

}

bool FRHIDrawTests::InternalDrawBaseVertexAndInstance(FRHICommandListImmediate& RHICmdList, EDrawKind DrawKind, const TCHAR* TestName)
{
	FDrawTestResources Resources(RHICmdList);

	RHICmdList.ClearUAVUint(Resources.OutputBufferUAV, FUintVector4(0));

	FRHITexture* ColorRTs[1] = { Resources.RenderTarget.GetReference() };
	FRHIRenderPassInfo RenderPassInfo(1, ColorRTs, ERenderTargetActions::DontLoad_DontStore);

	const uint32 FirstInvalidVertex = Resources.NumValidVertices;

	const FRHIDrawIndexedIndirectParameters DrawArgs[6] =
	{
		// IndexCountPerInstance, InstanceCount, StartIndexLocation, BaseVertexLocation, StartInstanceLocation
		{3,                       1,             0,                  0,                  0}, // InstanceID = 0, expected to be drawn. No vertex of instance offset
		{3,                       1,             0,                  0,                  1}, // InstanceID = 1, expected to be drawn
		{3,                       1,             0,                  FirstInvalidVertex, 2}, // InstanceID = 2, expected to be culled by rendering a degenerate triangle (see vertex buffer initialization)
		{3,                       1,             3,                  0,                  3}, // InstanceID = 3, expected to be drawn (vertices 3,4,5 via index buffer offset)
		{3,                       1,             0,                  3,                  4}, // InstanceID = 4, expected to be drawn (vertices 3,4,5 via base vertex)
		{3,                       1,             3,                  3,                  5}, // InstanceID = 5, expected to be drawn (vertices 6,7,8 via base vertex and index buffer offset)
	};

	const uint32 ExpectedDrawnInstances[Resources.MaxInstances] = { 1, 1, 0, 1, 1, 1, 0, 0 };

	FBufferRHIRef DrawArgBuffer;
	if (DrawKind == EDrawKind::Indirect)
	{
		DrawArgBuffer = UE::RHIResourceUtils::CreateBufferFromArray(
			RHICmdList,
			TEXT("InternalDrawBaseVertexAndInstance_DrawArgs"), 
			EBufferUsageFlags::DrawIndirect | EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::VertexBuffer,
			ERHIAccess::IndirectArgs,
			MakeConstArrayView(DrawArgs)
		);
	}

	RHICmdList.Transition(FRHITransitionInfo(Resources.OutputBufferUAV, ERHIAccess::UAVCompute, ERHIAccess::UAVGraphics, EResourceTransitionFlags::None));
	RHICmdList.BeginUAVOverlap(); // Output UAV can be written without syncs between draws (each draw is expected to write into different slots)

	RHICmdList.BeginRenderPass(RenderPassInfo, TestName);
	RHICmdList.SetViewport(0, 0, 0, float(Resources.RenderTargetSize.X), float(Resources.RenderTargetSize.Y), 1);

	RHICmdList.ApplyCachedRenderTargets(Resources.GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, Resources.GraphicsPSOInit, 0);

	RHICmdList.SetStreamSource(0, Resources.VertexBuffer, 0);
	RHICmdList.SetStreamSource(1, Resources.InstanceIDBuffer, 0);

	FRHIBatchedShaderParameters& ShaderParameters = RHICmdList.GetScratchShaderParameters();
	SetUAVParameter(ShaderParameters, Resources.PixelShader->OutDrawnInstances, Resources.OutputBufferUAV);
	RHICmdList.SetBatchedShaderParameters(Resources.PixelShader.GetPixelShader(), ShaderParameters);

	if (DrawKind == EDrawKind::Direct)
	{
		for (const FRHIDrawIndexedIndirectParameters& Draw : DrawArgs)
		{
			RHICmdList.DrawIndexedPrimitive(Resources.IndexBuffer,
				Draw.BaseVertexLocation,
				Draw.StartInstanceLocation,
				Draw.IndexCountPerInstance,
				Draw.StartIndexLocation,
				Draw.IndexCountPerInstance / 3 /*NumPrimitives*/,
				Draw.InstanceCount);
		}
	}
	else
	{
		for (const FRHIDrawIndexedIndirectParameters& Draw : DrawArgs)
		{
			const uint32 ArgumentOffset = uint32(uint64(&Draw) - uint64(DrawArgs));
			RHICmdList.DrawIndexedPrimitiveIndirect(Resources.IndexBuffer, DrawArgBuffer, ArgumentOffset);
		}
	}

	RHICmdList.EndRenderPass();

	RHICmdList.EndUAVOverlap();

	RHICmdList.Transition(FRHITransitionInfo(Resources.OutputBufferUAV, ERHIAccess::UAVGraphics, ERHIAccess::CopySrc, EResourceTransitionFlags::None));

	TConstArrayView<uint8> ExpectedOutputView = MakeArrayView(reinterpret_cast<const uint8*>(ExpectedDrawnInstances), sizeof(ExpectedDrawnInstances));
	bool bSucceeded = FRHIBufferTests::VerifyBufferContents(TestName, RHICmdList, Resources.OutputBuffer, ExpectedOutputView);

	return bSucceeded;
}

bool FRHIDrawTests::Test_DrawBaseVertexAndInstanceDirect(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.SupportsBaseVertexIndex || !GRHIGlobals.SupportsFirstInstance)
	{
		return true;
	}

	return InternalDrawBaseVertexAndInstance(RHICmdList, EDrawKind::Direct, TEXT("Test_DrawBaseVertexAndInstanceDirect"));
}

bool FRHIDrawTests::Test_DrawBaseVertexAndInstanceIndirect(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.SupportsBaseVertexIndex || !GRHIGlobals.SupportsFirstInstance)
	{
		return true;
	}

	return InternalDrawBaseVertexAndInstance(RHICmdList, EDrawKind::Indirect, TEXT("Test_DrawBaseVertexAndInstanceIndirect"));
}


bool FRHIDrawTests::Test_MultiDrawIndirect(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.SupportsMultiDrawIndirect)
	{
		return true;
	}

	// Probably could/should automatically enable in the outer scope when running RHI Unit Tests
	// RenderCaptureInterface::FScopedCapture RenderCapture(true /*bEnable*/, &RHICmdList, TEXT("Test_MultiDrawIndirect"));

	FDrawTestResources Resources(RHICmdList);

	const uint32 CountValues[4] = { 1, 1, 16, 0 };
	FBufferRHIRef CountBuffer = UE::RHIResourceUtils::CreateBufferFromArray(
		RHICmdList,
		TEXT("Test_MultiDrawIndirect_Count"),
		EBufferUsageFlags::DrawIndirect | EBufferUsageFlags::UnorderedAccess,
		ERHIAccess::IndirectArgs,
		MakeConstArrayView(CountValues)
	);

	const FRHIDrawIndexedIndirectParameters DrawArgs[] =
	{
		// IndexCountPerInstance, InstanceCount, StartIndexLocation, BaseVertexLocation, StartInstanceLocation
		{3, 1, 0, 0, 0}, // fill slot 0
		// gap in slot 1
		{3, 2, 0, 0, 2}, // fill slots 2, 3 using 1 sub-draw
		// gap in slot 4
		{3, 1, 0, 0, 5}, // fill slots 5, 6 using 2 sub-draws
		{3, 1, 0, 0, 6},
		{3, 1, 0, 0, 7}, // this draw is expected to never execute
	};

	const uint32 ExpectedDrawnInstances[Resources.MaxInstances] = { 1, 0, 1, 1, 0, 1, 1, 0 };

	FBufferRHIRef DrawArgBuffer = UE::RHIResourceUtils::CreateBufferFromArray(
		RHICmdList,
		TEXT("Test_MultiDrawIndirect_DrawArgs"),
		EBufferUsageFlags::DrawIndirect | EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::VertexBuffer,
		ERHIAccess::IndirectArgs,
		MakeConstArrayView(DrawArgs)
	);

	RHICmdList.ClearUAVUint(Resources.OutputBufferUAV, FUintVector4(0));

	FRHITexture* ColorRTs[1] = { Resources.RenderTarget.GetReference() };
	FRHIRenderPassInfo RenderPassInfo(1, ColorRTs, ERenderTargetActions::DontLoad_DontStore);

	RHICmdList.Transition(FRHITransitionInfo(Resources.OutputBufferUAV, ERHIAccess::UAVCompute, ERHIAccess::UAVGraphics, EResourceTransitionFlags::None));
	RHICmdList.BeginUAVOverlap(); // Output UAV can be written without syncs between draws (each draw is expected to write into different slots)

	RHICmdList.BeginRenderPass(RenderPassInfo, TEXT("Test_MultiDrawIndirect"));
	RHICmdList.SetViewport(0, 0, 0, float(Resources.RenderTargetSize.X), float(Resources.RenderTargetSize.Y), 1);

	RHICmdList.ApplyCachedRenderTargets(Resources.GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, Resources.GraphicsPSOInit, 0);

	RHICmdList.SetStreamSource(0, Resources.VertexBuffer, 0);
	RHICmdList.SetStreamSource(1, Resources.InstanceIDBuffer, 0);

	FRHIBatchedShaderParameters& ShaderParameters = RHICmdList.GetScratchShaderParameters();
	SetUAVParameter(ShaderParameters, Resources.PixelShader->OutDrawnInstances, Resources.OutputBufferUAV);
	RHICmdList.SetBatchedShaderParameters(Resources.PixelShader.GetPixelShader(), ShaderParameters);

	const uint32 DrawArgsStride = sizeof(DrawArgs[0]);
	const uint32 CountStride = sizeof(CountValues[0]);

	RHICmdList.MultiDrawIndexedPrimitiveIndirect(Resources.IndexBuffer, 
		DrawArgBuffer, DrawArgsStride*0, // 1 sub-draw with instance index 0
		CountBuffer, CountStride*0, // count buffer contains 1 in this slot
		5 // expect to draw only 1 instance due to GPU-side upper bound
	);

	RHICmdList.MultiDrawIndexedPrimitiveIndirect(Resources.IndexBuffer,
		DrawArgBuffer, DrawArgsStride*1, // 1 sub-draw with 2 instances at base index 2
		CountBuffer, CountStride*1, // count buffer contains 1 in this slot
		4 // expect to draw only 1 instance due to GPU-side upper bound
	);

	RHICmdList.MultiDrawIndexedPrimitiveIndirect(Resources.IndexBuffer,
		DrawArgBuffer, DrawArgsStride*2, // 2 sub-draws with 1 instance each starting at base index 5
		CountBuffer, CountStride*2, // count buffer contains 16 in this slot
		2 // expect to draw only 2 instances due to CPU-side upper bound
	);

	RHICmdList.MultiDrawIndexedPrimitiveIndirect(Resources.IndexBuffer,
		DrawArgBuffer, DrawArgsStride*4, // 1 sub-draw with 1 instance each starting at base index 7
		CountBuffer, CountStride*3, // count buffer contains 0 in this slot
		1 // expect to skip the draw due to GPU-side count of 0
	);

	RHICmdList.MultiDrawIndexedPrimitiveIndirect(Resources.IndexBuffer,
		DrawArgBuffer, DrawArgsStride*4, // 1 sub-draw with 1 instance each starting at base index 7
		CountBuffer, CountStride*0, // count buffer contains 1 in this slot
		0 // expect to skip the draw due to CPU-side count of 0
	);

	RHICmdList.EndRenderPass();

	RHICmdList.EndUAVOverlap();

	RHICmdList.Transition(FRHITransitionInfo(Resources.OutputBufferUAV, ERHIAccess::UAVGraphics, ERHIAccess::CopySrc, EResourceTransitionFlags::None));

	TConstArrayView<uint8> ExpectedOutputView = MakeArrayView(reinterpret_cast<const uint8*>(ExpectedDrawnInstances), sizeof(ExpectedDrawnInstances));
	bool bSucceeded = FRHIBufferTests::VerifyBufferContents(TEXT("Test_MultiDrawIndirect"), RHICmdList, Resources.OutputBuffer, ExpectedOutputView);

	return bSucceeded;
}
