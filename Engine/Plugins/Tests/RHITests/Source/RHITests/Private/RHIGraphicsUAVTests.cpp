// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIGraphicsUAVTests.h"
#include "RHIBufferTests.h" // for VerifyBufferContents
#include "RHIStaticStates.h"
#include "ShaderCompilerCore.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"

class FTestGraphicsUAVTrivialVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTestGraphicsUAVTrivialVS);
	FTestGraphicsUAVTrivialVS() = default;
	FTestGraphicsUAVTrivialVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
	{
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}
};
IMPLEMENT_GLOBAL_SHADER(FTestGraphicsUAVTrivialVS, "/Plugin/RHITests/Private/TestGraphicsUAV.usf", "TestGraphicsUAVTrivialMainVS", SF_Vertex);

class FTestGraphicsUAVWriteVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTestGraphicsUAVWriteVS);
	FTestGraphicsUAVWriteVS() = default;
	FTestGraphicsUAVWriteVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
	{
		RWVertexShaderOutput.Bind(Initializer.ParameterMap, TEXT("RWVertexShaderOutput"), SPF_Mandatory);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}
	LAYOUT_FIELD(FShaderResourceParameter, RWVertexShaderOutput);
};
IMPLEMENT_GLOBAL_SHADER(FTestGraphicsUAVWriteVS, "/Plugin/RHITests/Private/TestGraphicsUAV.usf", "TestGraphicsUAVWriteMainVS", SF_Vertex);

class FTestGraphicsUAVWritePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FTestGraphicsUAVWritePS);
	FTestGraphicsUAVWritePS() = default;
	FTestGraphicsUAVWritePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
	: FGlobalShader(Initializer)
	{
		RWPixelShaderOutput.Bind(Initializer.ParameterMap, TEXT("RWPixelShaderOutput"), SPF_Mandatory);
	}
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
	}
	LAYOUT_FIELD(FShaderResourceParameter, RWPixelShaderOutput);
};
IMPLEMENT_GLOBAL_SHADER(FTestGraphicsUAVWritePS, "/Plugin/RHITests/Private/TestGraphicsUAV.usf", "TestGraphicsUAVWriteMainPS", SF_Pixel);

static void Test_GraphicsUAV_Common(FRHICommandListImmediate& RHICmdList,
	FRHIVertexShader* VertexShaderRHI, FRHIPixelShader* PixelShaderRHI,
	TFunctionRef<void(FRHICommandListImmediate& RHICmdList)> RenderCallback)
{
	FIntPoint RenderTargetSize = FIntPoint(4, 4);
	FTextureRHIRef RenderTarget;

	FGraphicsPipelineStateInitializer GraphicsPSOInit;

	FRHITextureDesc RenderTargetTextureDesc(ETextureDimension::Texture2D,
		ETextureCreateFlags::RenderTargetable, PF_B8G8R8A8, FClearValueBinding(),
		RenderTargetSize, 1, 1, 1, 1, 0);

	FRHITextureCreateDesc RenderTargetCreateDesc(RenderTargetTextureDesc,
		ERHIAccess::RTV, TEXT("GraphicsUAVTests_RenderTarget"));
	RenderTarget = RHICmdList.CreateTexture(RenderTargetCreateDesc);

	FVertexDeclarationElementList VertexDeclarationElements;
	VertexDeclarationElements.Add(FVertexElement(0, 0, VET_Float4, 0, 16));

	FVertexDeclarationRHIRef VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(VertexDeclarationElements);

	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShaderRHI;
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = VertexDeclarationRHI;
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShaderRHI;
	GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.PrimitiveType = EPrimitiveType::PT_TriangleList;

	TArray<FVector4f> Vertices;
	Vertices.Reserve(3);
	Vertices.Add(FVector4f(-1.0f, -1.0f, 0.0f, 1.0f));
	Vertices.Add(FVector4f(-1.0f, +3.0f, 0.0f, 1.0f));
	Vertices.Add(FVector4f(+3.0f, -1.0f, 0.0f, 1.0f));

	FBufferRHIRef VertexBuffer = UE::RHIResourceUtils::CreateVertexBufferFromArray(RHICmdList, TEXT("GraphicsUAVTests_VertexBuffer"), MakeConstArrayView(Vertices));

	FRHITexture* ColorRTs[1] = { RenderTarget.GetReference() };
	FRHIRenderPassInfo RenderPassInfo(1, ColorRTs, ERenderTargetActions::DontLoad_DontStore);

	RHICmdList.BeginRenderPass(RenderPassInfo, TEXT("GraphicsUAVTest"));
	RHICmdList.SetViewport(0, 0, 0, float(RenderTargetSize.X), float(RenderTargetSize.Y), 1);

	RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
	SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

	RHICmdList.SetStreamSource(0, VertexBuffer, 0);

	RenderCallback(RHICmdList);

	RHICmdList.EndRenderPass();
}

bool FRHIGraphicsUAVTests::Test_GraphicsUAV_PixelShader(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.SupportsPixelShaderUAVs)
	{
		return true;
	}

	TShaderMapRef<FTestGraphicsUAVTrivialVS> VertexShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	TShaderMapRef<FTestGraphicsUAVWritePS> PixelShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	static constexpr uint32 MaxInstances = 8;
	static constexpr uint32 OutputBufferStride = sizeof(uint32);
	static constexpr uint32 OutputBufferSize = OutputBufferStride * MaxInstances;
	constexpr EBufferUsageFlags OutputBufferUsage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy | EBufferUsageFlags::StructuredBuffer;

	// NOTE: using a structured buffer here as workaround for UE-212251
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(TEXT("GraphicsUAVTests_PixelShaderOutput"), OutputBufferSize, OutputBufferStride, OutputBufferUsage)
		.SetInitialState(ERHIAccess::UAVCompute);
	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(CreateDesc);

	FUnorderedAccessViewRHIRef OutputBufferUAV = RHICmdList.CreateUnorderedAccessView(OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(4));

	RHICmdList.ClearUAVUint(OutputBufferUAV, FUintVector4(~0u));

	RHICmdList.Transition(FRHITransitionInfo(OutputBufferUAV, ERHIAccess::UAVCompute, ERHIAccess::UAVGraphics, EResourceTransitionFlags::None));

	Test_GraphicsUAV_Common(RHICmdList,
		VertexShader.GetVertexShader(),
		PixelShader.GetPixelShader(),
		[&PixelShader, &OutputBufferUAV](FRHICommandListImmediate& RHICmdList)
	{
			FRHIBatchedShaderParameters& ShaderParameters = RHICmdList.GetScratchShaderParameters();
			SetUAVParameter(ShaderParameters, PixelShader->RWPixelShaderOutput, OutputBufferUAV);
			RHICmdList.SetBatchedShaderParameters(PixelShader.GetPixelShader(), ShaderParameters);
			RHICmdList.DrawPrimitive(0, 1, MaxInstances);
	});

	RHICmdList.Transition(FRHITransitionInfo(OutputBufferUAV, ERHIAccess::UAVGraphics, ERHIAccess::CopySrc, EResourceTransitionFlags::None));

	// Expect pixel shader to populate UAV with instance IDs

	const uint32 ExpectedOutput[MaxInstances] = { 0, 1, 2, 3, 4, 5, 6, 7 };
	TConstArrayView<uint8> ExpectedOutputView = MakeArrayView(reinterpret_cast<const uint8*>(ExpectedOutput), sizeof(ExpectedOutput));
	const bool bSucceeded = FRHIBufferTests::VerifyBufferContents(TEXT("GraphicsUAV_PixelShader"), RHICmdList, OutputBuffer, ExpectedOutputView);

	return bSucceeded;
}

bool FRHIGraphicsUAVTests::Test_GraphicsUAV_VertexShader(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.SupportsVertexShaderUAVs)
	{
		return true;
	}

	TShaderMapRef<FTestGraphicsUAVWriteVS> VertexShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	static const uint32 MaxVertices = 3;
	static constexpr uint32 OutputBufferStride = sizeof(uint32);
	static constexpr uint32 OutputBufferSize = OutputBufferStride * MaxVertices;
	constexpr EBufferUsageFlags OutputBufferUsage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy | EBufferUsageFlags::StructuredBuffer;

	// NOTE: using a structured buffer here as workaround for UE-212251
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(TEXT("GraphicsUAVTests_VertexShaderOutput"), OutputBufferSize, OutputBufferStride, OutputBufferUsage)
		.SetInitialState(ERHIAccess::UAVCompute);

	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(CreateDesc);

	FUnorderedAccessViewRHIRef OutputBufferUAV = RHICmdList.CreateUnorderedAccessView(OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
		.SetType(FRHIViewDesc::EBufferType::Structured)
		.SetStride(4));

	RHICmdList.ClearUAVUint(OutputBufferUAV, FUintVector4(~0u));

	RHICmdList.Transition(FRHITransitionInfo(OutputBufferUAV, ERHIAccess::UAVCompute, ERHIAccess::UAVGraphics, EResourceTransitionFlags::None));

	Test_GraphicsUAV_Common(RHICmdList,
		VertexShader.GetVertexShader(),
		nullptr, // vertex-only rendering
	[&VertexShader, &OutputBufferUAV](FRHICommandListImmediate& RHICmdList)
	{
		FRHIBatchedShaderParameters& ShaderParameters = RHICmdList.GetScratchShaderParameters();
		SetUAVParameter(ShaderParameters, VertexShader->RWVertexShaderOutput, OutputBufferUAV);
		RHICmdList.SetBatchedShaderParameters(VertexShader.GetVertexShader(), ShaderParameters);
		RHICmdList.DrawPrimitive(0, 1, 1);
	});

	RHICmdList.Transition(FRHITransitionInfo(OutputBufferUAV, ERHIAccess::UAVGraphics, ERHIAccess::CopySrc, EResourceTransitionFlags::None));

	// Expect vertex shader to populate UAV with vertex IDs

	const uint32 ExpectedOutput[MaxVertices] = { 0, 1, 2 };
	TConstArrayView<uint8> ExpectedOutputView = MakeArrayView(reinterpret_cast<const uint8*>(ExpectedOutput), sizeof(ExpectedOutput));
	const bool bSucceeded = FRHIBufferTests::VerifyBufferContents(TEXT("GraphicsUAV_VertexShader"), RHICmdList, OutputBuffer, ExpectedOutputView);

	return bSucceeded;
}

