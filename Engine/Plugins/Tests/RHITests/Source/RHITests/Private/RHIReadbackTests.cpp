// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIReadbackTests.h"
#include "CommonRenderResources.h"
#include "RHIFeatureLevel.h"
#include "ShaderParameterMacros.h"
#include "RenderGraphUtils.h"
#include "RHIGPUReadback.h"

struct FStructuredBufferElem
{
	uint32	IntVal;
	float	FloatVal;
};

class FFillBufferCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FFillBufferCS);
	SHADER_USE_PARAMETER_STRUCT(FFillBufferCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSizeX = 64;
	static constexpr uint32 ThreadGroupSizeY = 1;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), ThreadGroupSizeY);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWBuffer<uint>, TypedBufferUAV)
		SHADER_PARAMETER_UAV(RWByteAddressBuffer, ByteAddressBufferUAV)
		SHADER_PARAMETER_UAV(RWStructuredBuffer<FStructuredBufferElem>, StructuredBufferUAV)
		SHADER_PARAMETER(uint32, NumElements)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FFillBufferCS, "/Plugin/RHITests/Private/TestFillBuffer.usf", "FillBuffers", SF_Compute);

class FFillTexturesCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FFillTexturesCS);
	SHADER_USE_PARAMETER_STRUCT(FFillTexturesCS, FGlobalShader);

	static constexpr uint32 ThreadGroupSizeX = 8;
	static constexpr uint32 ThreadGroupSizeY = 8;

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_X"), ThreadGroupSizeX);
		OutEnvironment.SetDefine(TEXT("THREADGROUP_SIZE_Y"), ThreadGroupSizeY);
	}

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWTexture2D<float3>, Texture2DUAV)
		SHADER_PARAMETER_UAV(RWTexture3D<float3>, Texture3DUAV)
		SHADER_PARAMETER_UAV(RWTexture2DArray<float3>, TextureArrayUAV)
		SHADER_PARAMETER(FUintVector3, TextureSize)
	END_SHADER_PARAMETER_STRUCT()
};

IMPLEMENT_GLOBAL_SHADER(FFillTexturesCS, "/Plugin/RHITests/Private/TestFillBuffer.usf", "FillTextures", SF_Compute);

bool FRHIReadbackTests::Test_BufferReadback(FRHICommandListImmediate& RHICmdList)
{
	const uint32 NumElements = 40;
	const uint32 TypedBufferSize = NumElements * sizeof(uint32);
	const uint32 ByteAddressBufferSize = NumElements * sizeof(uint32);
	const uint32 StructuredBufferSize = NumElements * sizeof(FStructuredBufferElem);

	FBufferRHIRef TypedBuffer, ByteAddressBuffer, StructuredBuffer;
	FUnorderedAccessViewRHIRef TypedBufferUAV, ByteAddressBufferUAV, StructuredBufferUAV;

	//
	// Create typed, byte address and structured buffers.
	//
	{
		const EBufferUsageFlags TypedBufferUsage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy;
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("TypedBuffer"), TypedBufferSize, 0, TypedBufferUsage)
			.SetInitialState(ERHIAccess::UAVCompute);
		TypedBuffer = RHICmdList.CreateBuffer(CreateDesc);

		auto TypedBufferUAVCreateDesc = FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Typed)
			.SetFormat(PF_R32_UINT)
			.SetNumElements(NumElements);
		TypedBufferUAV = RHICmdList.CreateUnorderedAccessView(TypedBuffer, TypedBufferUAVCreateDesc);
	}

	{
		const EBufferUsageFlags ByteAddressBufferUsage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy | EBufferUsageFlags::ByteAddressBuffer | EBufferUsageFlags::StructuredBuffer;
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("ByteAddressBuffer"), ByteAddressBufferSize, sizeof(uint32), ByteAddressBufferUsage)
			.SetInitialState(ERHIAccess::UAVCompute);
		ByteAddressBuffer = RHICmdList.CreateBuffer(CreateDesc);

		auto ByteAddressBufferUAVCreateDesc = FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Raw);
		ByteAddressBufferUAV = RHICmdList.CreateUnorderedAccessView(ByteAddressBuffer, ByteAddressBufferUAVCreateDesc);
	}

	{
		const EBufferUsageFlags StructuredBufferUsage = EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy | EBufferUsageFlags::StructuredBuffer;
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("StructuredBuffer"), StructuredBufferSize, sizeof(FStructuredBufferElem), StructuredBufferUsage)
			.SetInitialState(ERHIAccess::UAVCompute);
		StructuredBuffer = RHICmdList.CreateBuffer(CreateDesc);

		auto StructuredBufferUAVCreateDesc = FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(sizeof(FStructuredBufferElem))
			.SetNumElements(NumElements);
		StructuredBufferUAV = RHICmdList.CreateUnorderedAccessView(StructuredBuffer, StructuredBufferUAVCreateDesc);
	}

	//
	// Fill the buffers with data.
	//
	{
		TShaderMapRef<FFillBufferCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
		const uint32 NumThreadGroups = FMath::DivideAndRoundUp(NumElements, FFillBufferCS::ThreadGroupSizeX);

		FFillBufferCS::FParameters Parameters;
		Parameters.TypedBufferUAV = TypedBufferUAV;
		Parameters.ByteAddressBufferUAV = ByteAddressBufferUAV;
		Parameters.StructuredBufferUAV = StructuredBufferUAV;
		Parameters.NumElements = NumElements;

		FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(NumThreadGroups, 1, 1));

		RHICmdList.Transition({
			FRHITransitionInfo(TypedBufferUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc),
			FRHITransitionInfo(ByteAddressBufferUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc),
			FRHITransitionInfo(StructuredBufferUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc)
			});
	}

	// Kick off the readbacks.
	FRHIGPUBufferReadback TypedBufferReadback(TEXT("TypedBufferReadback"));
	TypedBufferReadback.EnqueueCopy(RHICmdList, TypedBuffer, TypedBufferSize);

	FRHIGPUBufferReadback ByteAddressBufferReadback(TEXT("ByteAddressBufferReadback"));
	ByteAddressBufferReadback.EnqueueCopy(RHICmdList, ByteAddressBuffer, ByteAddressBufferSize);

	FRHIGPUBufferReadback StructuredBufferReadback(TEXT("StructuredBufferReadback"));
	StructuredBufferReadback.EnqueueCopy(RHICmdList, StructuredBuffer, StructuredBufferSize);

	// Sync the GPU. Unfortunately we can't use the fences because not all RHIs implement them yet.
	constexpr bool bFlushResources = true;
	RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

	// Check the contents of the typed buffer.
	bool bTypedBufferResult = true;

	{
		const uint32* TypedBufferData = (const uint32*)TypedBufferReadback.Lock(TypedBufferSize);
		for (uint32 Index = 0; Index < NumElements; ++Index)
		{
			const uint32 ExpectedVal = Index;
			if (TypedBufferData[Index] != ExpectedVal)
			{
				UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Typed Buffer Readback\"\n\nIndex: %u; Expected Data: %u; Actual Data: %u\n\n", Index, ExpectedVal, TypedBufferData[Index]);
				bTypedBufferResult = false;
				break;
			}
		}

		if (bTypedBufferResult)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"Typed Buffer Readback\"");
		}

		TypedBufferReadback.Unlock();
	}

	// Check the contents of the byte address buffer.
	bool bByteAddressBufferResult = true;

	{
		const uint32* ByteAddressBufferData = (const uint32*)ByteAddressBufferReadback.Lock(ByteAddressBufferSize);
		for (uint32 Index = 0; Index < NumElements; ++Index)
		{
			const uint32 ExpectedVal = Index + 1024;
			if (ByteAddressBufferData[Index] != ExpectedVal)
			{
				UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Byte Address Buffer Readback\"\n\nIndex: %u; Expected Data: %u; Actual Data: %u\n\n", Index, ExpectedVal, ByteAddressBufferData[Index]);
				bByteAddressBufferResult = false;
				break;
			}
		}

		if (bByteAddressBufferResult)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"Byte Address Buffer Readback\"");
		}

		ByteAddressBufferReadback.Unlock();
	}

	// Check the contents of the structured buffer.
	bool bStructuredBufferResult = true;

	{
		const FStructuredBufferElem* StructuredBufferData = (const FStructuredBufferElem*)StructuredBufferReadback.Lock(StructuredBufferSize);
		for (uint32 Index = 0; Index < NumElements; ++Index)
		{
			const uint32 ExpectedIntVal = Index + 2048;
			const float ExpectedFloatVal = Index + 4096;
			if (StructuredBufferData[Index].IntVal != ExpectedIntVal || StructuredBufferData[Index].FloatVal != ExpectedFloatVal)
			{
				UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Structured Buffer Readback\"\n\nIndex: %u; Expected Int Data: %u; Actual Int Data: %u; Expected Float Data: %.2f; Actual Float Data: %.2f\n\n",
					Index, ExpectedIntVal, StructuredBufferData[Index].IntVal, ExpectedFloatVal, StructuredBufferData[Index].FloatVal);
				bStructuredBufferResult = false;
				break;
			}
		}

		if (bStructuredBufferResult)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"Structured Buffer Readback\"");
		}

		StructuredBufferReadback.Unlock();
	}

	return bTypedBufferResult && bByteAddressBufferResult && bStructuredBufferResult;
}

bool FRHIReadbackTests::Test_TextureReadback(FRHICommandListImmediate& RHICmdList)
{
	const FUintVector3 TextureSize(521, 491, 5);

	FTextureRHIRef Texture2D, Texture3D, TextureArray;
	FUnorderedAccessViewRHIRef Texture2DUAV, Texture3DUAV, TextureArrayUAV;

	//
	// Create one texture of each kind.
	//
	{
		const FRHITextureCreateDesc Texture2DDesc =
			FRHITextureCreateDesc::Create2D(TEXT("Texture2D"))
			.SetFlags(ETextureCreateFlags::UAV)
			.SetExtent(TextureSize.X, TextureSize.Y)
			.SetNumMips(1)
			.SetFormat(PF_R32_UINT)
			.SetInitialState(ERHIAccess::UAVCompute);

		Texture2D = RHICmdList.CreateTexture(Texture2DDesc);
		auto Texture2DUAVDesc = FRHIViewDesc::CreateTextureUAV()
			.SetDimensionFromTexture(Texture2D)
			.SetFormat(PF_R32_UINT);

		Texture2DUAV = RHICmdList.CreateUnorderedAccessView(Texture2D, Texture2DUAVDesc);
	}

	{
		const FRHITextureCreateDesc Texture3DDesc =
			FRHITextureCreateDesc::Create3D(TEXT("Texture3D"))
			.SetFlags(ETextureCreateFlags::UAV)
			.SetExtent(TextureSize.X, TextureSize.Y)
			.SetDepth(TextureSize.Z)
			.SetNumMips(1)
			.SetFormat(PF_R32_UINT)
			.SetInitialState(ERHIAccess::UAVCompute);

		Texture3D = RHICmdList.CreateTexture(Texture3DDesc);
		auto Texture3DUAVDesc = FRHIViewDesc::CreateTextureUAV()
			.SetDimensionFromTexture(Texture3D)
			.SetFormat(PF_R32_UINT);

		Texture3DUAV = RHICmdList.CreateUnorderedAccessView(Texture3D, Texture3DUAVDesc);
	}

	{
		const FRHITextureCreateDesc TextureArrayDesc =
			FRHITextureCreateDesc::Create2DArray(TEXT("TextureArray"))
			.SetFlags(ETextureCreateFlags::UAV)
			.SetExtent(TextureSize.X, TextureSize.Y)
			.SetArraySize(TextureSize.Z)
			.SetFormat(PF_R32_UINT)
			.SetInitialState(ERHIAccess::UAVCompute);

		TextureArray = RHICmdList.CreateTexture(TextureArrayDesc);
		auto TextureArrayUAVDesc = FRHIViewDesc::CreateTextureUAV()
			.SetDimensionFromTexture(TextureArray)
			.SetFormat(PF_R32_UINT);

		TextureArrayUAV = RHICmdList.CreateUnorderedAccessView(TextureArray, TextureArrayUAVDesc);
	}

	//
	// Fill the textures with data.
	//
	{
		TShaderMapRef<FFillTexturesCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
		const uint32 NumThreadGroupsX = FMath::DivideAndRoundUp(TextureSize.X, FFillTexturesCS::ThreadGroupSizeX);
		const uint32 NumThreadGroupsY = FMath::DivideAndRoundUp(TextureSize.Y, FFillTexturesCS::ThreadGroupSizeY);

		FFillTexturesCS::FParameters Parameters;
		Parameters.Texture2DUAV = Texture2DUAV;
		Parameters.Texture3DUAV = Texture3DUAV;
		Parameters.TextureArrayUAV = TextureArrayUAV;
		Parameters.TextureSize = TextureSize;

		FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(NumThreadGroupsX, NumThreadGroupsY, 1));

		RHICmdList.Transition({
			FRHITransitionInfo(Texture2DUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc),
			FRHITransitionInfo(Texture3DUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc),
			FRHITransitionInfo(TextureArrayUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc)
		});
	}

	// Kick off the readbacks. We'll use an offset to make sure that works correctly. Note that for arrays we only support reading back one slice at a time.
	FIntVector ReadbackOffset2D(17, 23, 0), ReadbackOffset3D(17, 23, 1), ReadbackOffsetArray(17, 23, 0);
	FIntVector ReadbackSize2D(293, 173, 1), ReadbackSize3D(293, 173, 3), ReadbackSizeArray(293, 173, 1);
	const uint32 ReadbackArraySlice = 2;

	FRHIGPUTextureReadback Texture2DReadback(TEXT("Texture2DReadback"));
	Texture2DReadback.EnqueueCopy(RHICmdList, Texture2D, ReadbackOffset2D, 0, ReadbackSize2D);

	//Some platform only support readback on single 2d texture therefore the test can happen only on one slice
	if (GRHIGlobals.SupportLinearTextureVolumeFormat == false)
	{
		ReadbackOffset3D.Z = 0;
		ReadbackSize3D.Z = 1;
	}

	FRHIGPUTextureReadback Texture3DReadback(TEXT("Texture3DReadback"));
	Texture3DReadback.EnqueueCopy(RHICmdList, Texture3D, ReadbackOffset3D, 0, ReadbackSize3D);

	FRHIGPUTextureReadback TextureArrayReadback(TEXT("TextureArrayReadback"));
	TextureArrayReadback.EnqueueCopy(RHICmdList, TextureArray, ReadbackOffsetArray, ReadbackArraySlice, ReadbackSizeArray);

	// Sync the GPU. Unfortunately we can't use the fences because not all RHIs implement them yet.
	constexpr bool bFlushResources = true;
	RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

	// Check the contents of the 2D texture.
	bool bTexture2DResult = true;

	{
		int32 RowPitchInPixels;
		const uint32* Texture2DData = (const uint32*)Texture2DReadback.Lock(RowPitchInPixels);
		const uint32* RowStart = Texture2DData;
		for (int32 Y = 0; Y < ReadbackSize2D.Y && bTexture2DResult; ++Y)
		{
			for (int32 X = 0; X < ReadbackSize2D.X; ++X)
			{
				const uint32 ExpectedVal = (Y + ReadbackOffset2D.Y) * TextureSize.X + (X + ReadbackOffset2D.X);
				if (RowStart[X] != ExpectedVal)
				{
					UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Texture 2D Readback\"\n\nCoords: (%u, %u); Expected Data: %u; Actual Data: %u\n\n", X, Y, ExpectedVal, RowStart[X]);
					bTexture2DResult = false;
					break;
				}
			}

			RowStart += RowPitchInPixels;
		}

		if (bTexture2DResult)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"Texture 2D Readback\"");
		}

		Texture2DReadback.Unlock();
	}

	// Check the contents of the 3D texture.
	bool bTexture3DResult = true;

	{
		int32 RowPitchInPixels, BufferHeight;
		const uint32* Texture3DData = (const uint32*)Texture3DReadback.Lock(RowPitchInPixels, &BufferHeight);
		const uint32* SliceStart = Texture3DData;
		for (int32 Z = 0; Z < ReadbackSize3D.Z && bTexture3DResult; ++Z)
		{
			const uint32* RowStart = SliceStart;
			for (int32 Y = 0; Y < ReadbackSize3D.Y && bTexture3DResult; ++Y)
			{
				for (int32 X = 0; X < ReadbackSize3D.X; ++X)
				{
					const uint32 ExpectedVal = (Y + ReadbackOffset3D.Y) * TextureSize.X + (X + ReadbackOffset3D.X) + (Z + ReadbackOffset3D.Z + 10) * 1024;
					if (RowStart[X] != ExpectedVal)
					{
						UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Texture 3D Readback\"\n\nCoords: (%u, %u, %u); Expected Data: %u; Actual Data: %u\n\n", X, Y, Z, ExpectedVal, RowStart[X]);
						bTexture3DResult = false;
						break;
					}
				}

				RowStart += RowPitchInPixels;
			}

			SliceStart += BufferHeight * RowPitchInPixels;
		}

		if (bTexture3DResult)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"Texture 3D Readback\"");
		}

		Texture3DReadback.Unlock();
	}

	// Check the contents of the array texture.
	bool bTextureArrayResult = true;

	{
		int32 RowPitchInPixels;
		const uint32* TextureArrayData = (const uint32*)TextureArrayReadback.Lock(RowPitchInPixels);
		const uint32* RowStart = TextureArrayData;
		for (int32 Y = 0; Y < ReadbackSizeArray.Y && bTextureArrayResult; ++Y)
		{
			for (int32 X = 0; X < ReadbackSizeArray.X; ++X)
			{
				const uint32 ExpectedVal = (Y + ReadbackOffsetArray.Y) * TextureSize.X + (X + ReadbackOffsetArray.X) + (ReadbackArraySlice + 10 + TextureSize.Z) * 1024;
				if (RowStart[X] != ExpectedVal)
				{
					UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"Texture Array Readback\"\n\nCoords: (%u, %u); Slice: %u; Expected Data: %u; Actual Data: %u\n\n", X, Y, ReadbackArraySlice, ExpectedVal, RowStart[X]);
					bTextureArrayResult = false;
					break;
				}
			}

			RowStart += RowPitchInPixels;
		}

		if (bTextureArrayResult)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"Texture Array Readback\"");
		}

		TextureArrayReadback.Unlock();
	}

	return bTexture2DResult && bTexture3DResult && bTextureArrayResult;
}
