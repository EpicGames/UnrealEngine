// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIReservedResourceTests.h"
#include "RHIBufferTests.h" // for VerifyBufferContents
#include "CommonRenderResources.h"
#include "RenderCaptureInterface.h"
#include "RHIStaticStates.h"

bool FRHIReservedResourceTests::Test_ReservedResource_CreateVolumeTexture(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.ReservedResources.SupportsVolumeTextures)
	{
		return true;
	}

	// Tiny volume texture with immediately committed physical memory.
	// Expected to exercise the packed mip layout case.

	{
		FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create3D(TEXT("TestTinyReservedTexture3D"));
		Desc.SetFlags(TexCreate_ShaderResource | TexCreate_ReservedResource | TexCreate_ImmediateCommit)
			.SetExtent(FIntPoint(8, 8))
			.SetDepth(8)
			.SetNumMips(1)
			.SetFormat(PF_A32B32G32R32F)
			.SetInitialState(ERHIAccess::SRVCompute);

		FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);
	}

	// Small size volume texture (16MB) with immediately committed physical memory.

	{
		FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create3D(TEXT("TestSmallReservedTexture3D"));
		Desc.SetFlags(TexCreate_ShaderResource | TexCreate_ReservedResource | TexCreate_ImmediateCommit)
			.SetExtent(FIntPoint(128, 128))
			.SetDepth(64)
			.SetNumMips(1)
			.SetFormat(PF_A32B32G32R32F)
			.SetInitialState(ERHIAccess::SRVCompute);

		FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);
	}

	// Medium size volume texture (128MB) with immediately committed physical memory.
	// Likely to be backed by multiple physical memory allocations.

	{
		FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create3D(TEXT("TestMediumReservedTexture3D"));
		Desc.SetFlags(TexCreate_ShaderResource | TexCreate_ReservedResource | TexCreate_ImmediateCommit)
			.SetExtent(FIntPoint(1024, 1024))
			.SetDepth(8)
			.SetNumMips(1)
			.SetFormat(PF_A32B32G32R32F)
			.SetInitialState(ERHIAccess::SRVCompute);

		FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);
	}

	// Huge volume texture without physical memory allocation.
	// Just test the ability to reserve a huge virtual address range.

	{
		FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create3D(TEXT("TestHugeReservedTexture3D"));
		Desc.SetFlags(TexCreate_ShaderResource | TexCreate_ReservedResource)
			.SetExtent(FIntPoint(2048, 2048))
			.SetDepth(2048)
			.SetNumMips(1)
			.SetFormat(PF_A32B32G32R32F)
			.SetInitialState(ERHIAccess::SRVCompute);

		FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);
	}

	return true;
}

bool FRHIReservedResourceTests::Test_ReservedResource_CreateTexture(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.ReservedResources.Supported)
	{
		return true;
	}

	const ETextureCreateFlags CreateFlags =
		  TexCreate_ShaderResource
		| TexCreate_ReservedResource
		| TexCreate_ImmediateCommit;

	const EPixelFormat TestFormats[] =
	{
		PF_DXT1,
		PF_R8,
		PF_R32_UINT,
	};

	struct FTestTextureConfig
	{
		FIntPoint Extent = FIntPoint(1,1);
		int32 NumMips = 1;
		int32 ArraySize = 1;
	};

	const int32 TileSize = GRHIGlobals.ReservedResources.TextureArrayMinimumMipDimension;

	FTestTextureConfig TestConfigs[] =
	{
		// Texture arrays
		FTestTextureConfig{.Extent = FIntPoint(TileSize * 8, TileSize),  .NumMips = 1, .ArraySize = 2},
		FTestTextureConfig{.Extent = FIntPoint(TileSize * 2, TileSize),  .NumMips = 1, .ArraySize = 3},

		// Regular textures
		FTestTextureConfig{.Extent = FIntPoint(4096, 128), .NumMips = 1, .ArraySize = 1},
		FTestTextureConfig{.Extent = FIntPoint(128, 128),  .NumMips = 1, .ArraySize = 1},
		FTestTextureConfig{.Extent = FIntPoint(512, 128),  .NumMips = 1, .ArraySize = 1},
		FTestTextureConfig{.Extent = FIntPoint(512, 128),  .NumMips = 1, .ArraySize = 1},
		FTestTextureConfig{.Extent = FIntPoint(4, 4),      .NumMips = 1, .ArraySize = 1},

#if 0 // TODO: as of 2023-10-18, reserved textures with mips are not supported
		FTestTextureConfig{.Extent = FIntPoint(128, 128),  .NumMips = 2, .ArraySize = 2},
		FTestTextureConfig{.Extent = FIntPoint(512, 128),  .NumMips = 2, .ArraySize = 2},
		FTestTextureConfig{.Extent = FIntPoint(512, 128),  .NumMips = 5, .ArraySize = 2},
		FTestTextureConfig{.Extent = FIntPoint(4, 4),      .NumMips = 2, .ArraySize = 2},
#endif
	};

	// Try to create resources of various formats and dimensions

	for (FTestTextureConfig Config : TestConfigs)
	{
		for (EPixelFormat Format : TestFormats)
		{
			FRHITextureCreateDesc Desc;

			if (Config.ArraySize > 1)
			{
				Desc = FRHITextureCreateDesc::Create2DArray(TEXT("TestReservedTexture2DArray"));
				Desc.SetArraySize(Config.ArraySize);
			}
			else
			{
				Desc = FRHITextureCreateDesc::Create2D(TEXT("TestReservedTexture2D"));
			}

			Desc.SetFlags(CreateFlags)
				.SetExtent(Config.Extent)
				.SetNumMips(Config.NumMips)
				.SetFormat(Format)
				.SetInitialState(ERHIAccess::SRVCompute);

			FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);
		}
	}

	// Try to create huge reserved textures without committing physical memory

	{
		FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create2D(TEXT("TestHugeReservedTexture2D"));
		Desc.SetFlags(TexCreate_ShaderResource | TexCreate_ReservedResource)
			.SetExtent(FIntPoint(16384, 16384))
			.SetNumMips(1)
			.SetFormat(PF_A32B32G32R32F)
			.SetInitialState(ERHIAccess::SRVCompute);

		FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);
	}

	return true;
}

bool FRHIReservedResourceTests::Test_ReservedResource_CreateTextureWithMips(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.ReservedResources.Supported)
	{
		return true;
	}

	// Try to create a 2D texture array with mips

	{
		FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create2DArray(TEXT("TestReservedTextureArray2DWithMips"));
		Desc.SetFlags(TexCreate_ShaderResource | TexCreate_RenderTargetable | TexCreate_ReservedResource | TexCreate_ImmediateCommit)
			.SetExtent(FIntPoint(1024, 1024))
			.SetArraySize(2)
			.SetNumMips(3)
			.SetFormat(PF_R8)
			.SetInitialState(ERHIAccess::SRVCompute);

		FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);
	}

	// Try to create a 2D texture with mips

	{
		FRHITextureCreateDesc Desc = FRHITextureCreateDesc::Create2D(TEXT("TestReservedTexture2DWithMips"));
		Desc.SetFlags(TexCreate_ShaderResource | TexCreate_ReservedResource | TexCreate_ImmediateCommit)
			.SetExtent(FIntPoint(128, 128))
			.SetNumMips(8) // full mip chain down to 1x1
			.SetFormat(PF_A32B32G32R32F)
			.SetInitialState(ERHIAccess::SRVCompute);

		FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);
	}

	return true;
}

bool FRHIReservedResourceTests::Test_ReservedResource_CreateBuffer(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.ReservedResources.Supported)
	{
		return true;
	}

	// Simply try to create reserved buffers of different types and sizes to see if we hit any unexpected paths in the RHI

	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("TestSmallReservedVertexBuffer"), 32768)
			.AddUsage(EBufferUsageFlags::ReservedResource)
			.SetInitialState(ERHIAccess::CopyDest);
		FBufferRHIRef Buffer = RHICmdList.CreateBuffer(CreateDesc);
	}

	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("TestSmallReservedUAV"), 32768, 4, EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource)
			.AddUsage(EBufferUsageFlags::ReservedResource)
			.SetInitialState(ERHIAccess::UAVGraphics);
		FBufferRHIRef Buffer = RHICmdList.CreateBuffer(CreateDesc);
	}

	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex(TEXT("TestReservedVertexBuffer"), 32 * 1024 * 1024)
			.AddUsage(EBufferUsageFlags::ReservedResource)
			.SetInitialState(ERHIAccess::CopyDest);
		FBufferRHIRef Buffer = RHICmdList.CreateBuffer(CreateDesc);
	}

	if (GRHIGlobals.RayTracing.Supported)
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("TestReservedAccelerationStructureBuffer"), 32 * 1024 * 1024, 4, EBufferUsageFlags::AccelerationStructure)
			.AddUsage(EBufferUsageFlags::ReservedResource)
			.SetInitialState(ERHIAccess::BVHWrite);
		FBufferRHIRef Buffer = RHICmdList.CreateBuffer(CreateDesc);
	}

	if (GRHIGlobals.RayTracing.Supported)
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("TestReservedRayTracingScratchBuffer"), 32 * 1024 * 1024, 4, EBufferUsageFlags::RayTracingScratch)
			.AddUsage(EBufferUsageFlags::ReservedResource)
			.SetInitialState(ERHIAccess::UAVCompute);
		FBufferRHIRef Buffer = RHICmdList.CreateBuffer(CreateDesc);
	}

	return true;
}

static void CommitBuffer(FRHICommandListImmediate& RHICmdList, FRHIBuffer* Buffer, uint64 CommitSize, ERHIAccess StateBefore, ERHIAccess StateAfter)
{
	FRHITransitionInfo TransitionInfo(Buffer, StateBefore, StateAfter, FRHICommitResourceInfo(CommitSize));

	TArrayView<const FRHITransitionInfo> TransitionInfos = MakeArrayView(&TransitionInfo, 1);

	FRHITransitionCreateInfo CreateInfo(
		ERHIPipeline::Graphics, ERHIPipeline::Graphics,
		ERHITransitionCreateFlags::None, TransitionInfos);

	const FRHITransition* Transition = RHICreateTransition(CreateInfo);

	RHICmdList.BeginTransition(Transition);
	RHICmdList.EndTransition(Transition);
}

struct FBufferVerificationRange
{
	// Offsets in bytes, uint32-aligned
	uint32 BeginByteOffset = 0;
	uint32 EndByteOffset = 0;

	uint32 ExpectedValue = 0;
};

static bool VerifyBufferRanges(const TCHAR* TestName, FRHICommandListImmediate& RHICmdList, FRHIBuffer* Buffer, TConstArrayView<FBufferVerificationRange> Ranges)
{
	return FRHIBufferTests::VerifyBufferContents(TestName, RHICmdList, MakeArrayView(&Buffer, 1),
		[Ranges](int32 /*BufferIndex*/, void* Ptr, uint32 NumBytes)
		{
			const uint32 Stride = sizeof(FBufferVerificationRange::ExpectedValue);
			const uint32 BufferNumElements = NumBytes / Stride;
			const uint32* BufferData = reinterpret_cast<const uint32*>(Ptr);

			for (const FBufferVerificationRange& Range : Ranges)
			{
				check(Range.BeginByteOffset % Stride == 0);
				check(Range.EndByteOffset % Stride == 0);

				const uint32 ElemBegin = Range.BeginByteOffset / Stride;
				const uint32 ElemEnd = Range.EndByteOffset / Stride;

				if (ElemBegin >= BufferNumElements || ElemEnd > BufferNumElements)
				{
					return false;
				}

				for (uint32 i = ElemBegin; i < ElemEnd; ++i)
				{
					if (BufferData[i] != Range.ExpectedValue)
					{
						return false;
					}
				}
			}

			return true;
		});
}


bool FRHIReservedResourceTests::Test_ReservedResource_CommitBuffer(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.ReservedResources.Supported)
	{
		return true;
	}

	const int32 TileSizeInBytes = GRHIGlobals.ReservedResources.TileSizeInBytes;
	const int32 BufferSizeInBytes = TileSizeInBytes * 128;

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(TEXT("TestReservedBufferExplicitCommit"), BufferSizeInBytes, 4, EBufferUsageFlags::ReservedResource | EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute);

	FBufferRHIRef Buffer = RHICmdList.CreateBuffer(CreateDesc);

	FUnorderedAccessViewRHIRef BufferUAV = RHICmdList.CreateUnorderedAccessView(Buffer,
		FRHIViewDesc::CreateBufferUAV()
		.SetType(FRHIViewDesc::EBufferType::Typed)
		.SetFormat(PF_R32_UINT));

	// Commit half of the resource, leaving the tail unmapped. 
	// The RHI follows D3D12 Tier 2 Reserved Resource semantics:
	// - Unmapped page writes are discarded
	// - Unmapped page reads return 0

	const int32 CommitSizeInBytes = BufferSizeInBytes / 2;
	CommitBuffer(RHICmdList, Buffer, CommitSizeInBytes, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute);

	const uint32 ClearValue = ~0u;
	RHICmdList.ClearUAVUint(BufferUAV, FUintVector4(ClearValue));

	RHICmdList.Transition(FRHITransitionInfo(Buffer, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));

	bool bSucceeded = VerifyBufferRanges(TEXT("Test_ReservedResource_CommitBuffer"), RHICmdList, Buffer,
		{
			{0,                 CommitSizeInBytes, ClearValue}, // First range is committed and expected to contain the value written by ClearUAVUint
			{CommitSizeInBytes, BufferSizeInBytes, 0u} // We follow the D3D convention for unmapped page access: writes are no-op, reads return 0
		});

	return bSucceeded;
}

bool FRHIReservedResourceTests::Test_ReservedResource_DecommitBuffer(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHIGlobals.ReservedResources.Supported)
	{
		return true;
	}

	const int32 TileSizeInBytes = GRHIGlobals.ReservedResources.TileSizeInBytes;
	const int32 BufferSizeInBytes = TileSizeInBytes * 128;

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(TEXT("TestReservedBufferExplicitCommit"), BufferSizeInBytes, 4, EBufferUsageFlags::ReservedResource | EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::ShaderResource | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute);

	FBufferRHIRef Buffer = RHICmdList.CreateBuffer(CreateDesc);

	FUnorderedAccessViewRHIRef BufferUAV = RHICmdList.CreateUnorderedAccessView(Buffer,
		FRHIViewDesc::CreateBufferUAV()
		.SetType(FRHIViewDesc::EBufferType::Typed)
		.SetFormat(PF_R32_UINT));

	// Commit the entire buffer using N commands (to force multiple backing physical memory allocations) and fill it with ~0u

	CommitBuffer(RHICmdList, Buffer, BufferSizeInBytes / 4, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute);
	CommitBuffer(RHICmdList, Buffer, BufferSizeInBytes / 2, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute);
	CommitBuffer(RHICmdList, Buffer, BufferSizeInBytes, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute);

	const uint32 ClearValue = ~0u;
	RHICmdList.ClearUAVUint(BufferUAV, FUintVector4(ClearValue));
	RHICmdList.Transition(FRHITransitionInfo(Buffer, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));

	FRHIBuffer* Buffers[] = { Buffer.GetReference() };

	// Check that the clear has fully initialzied the buffer contents

	bool bSucceeded = false;

	bSucceeded = VerifyBufferRanges(TEXT("Test_ReservedResource_DecommitBuffer_Init"), RHICmdList, Buffer,
		{
			{0, BufferSizeInBytes, ClearValue}
		});

	if (!bSucceeded)
	{
		return false;
	}

	// Partially decommit the buffer using multiple operations, leaving only one tile mapped. Reading from decommited regions is expected to return 0.
	CommitBuffer(RHICmdList, Buffer, BufferSizeInBytes - TileSizeInBytes, ERHIAccess::CopySrc, ERHIAccess::UAVCompute); // shrink
	CommitBuffer(RHICmdList, Buffer, TileSizeInBytes * 2, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute); // shrink
	CommitBuffer(RHICmdList, Buffer, TileSizeInBytes * 3, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute); // grow
	CommitBuffer(RHICmdList, Buffer, TileSizeInBytes, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute); // shrink to final size

	RHICmdList.ClearUAVUint(BufferUAV, FUintVector4(ClearValue)); // Expected to have no effect on contents of the buffer except the first tile
	RHICmdList.Transition(FRHITransitionInfo(Buffer, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));

	bSucceeded = VerifyBufferRanges(TEXT("Test_ReservedResource_DecommitBuffer_Partial"), RHICmdList, Buffer,
		{
			{0,               TileSizeInBytes, ClearValue}, // First tile expected to retain value
			{TileSizeInBytes, BufferSizeInBytes, 0u} // The rest of the buffer is expected to read 0s
		});

	if (!bSucceeded)
	{
		return false;
	}

	// Re-commit the buffer and verify that it is fully cleared again

	CommitBuffer(RHICmdList, Buffer, BufferSizeInBytes / 2, ERHIAccess::CopySrc, ERHIAccess::UAVCompute);
	CommitBuffer(RHICmdList, Buffer, BufferSizeInBytes, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute);

	const uint32 ClearValue2 = 0x11223344;

	RHICmdList.ClearUAVUint(BufferUAV, FUintVector4(ClearValue2));
	RHICmdList.Transition(FRHITransitionInfo(Buffer, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));

	bSucceeded = VerifyBufferRanges(TEXT("Test_ReservedResource_DecommitBuffer_Init"), RHICmdList, Buffer,
		{
			{0, BufferSizeInBytes, ClearValue2}
		});

	return bSucceeded;
}

