// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIRootConstantsTests.h"

#include "RHI.h"
#include "RHIGPUReadback.h"
#include "RHIResourceUtils.h"
#include "RHIStaticStates.h"
#include "RHITestsCommon.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"
#include "PipelineStateCache.h"
#include "RenderGraphUtils.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterStruct.h"
#include "CommonRenderResources.h"

// ---- Shaders ----

class FRootConstantsTestShaderBase : public FGlobalShader
{
public:
	FRootConstantsTestShaderBase() = default;
	FRootConstantsTestShaderBase(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}
	
	static bool ShouldCompilePermutation(const FShaderPermutationParameters& Parameters)
	{
		return FDataDrivenShaderPlatformInfo::GetSupportsShaderRootConstants(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
	}
};

class FRootConstantsTestVS : public FRootConstantsTestShaderBase
{
public:
	DECLARE_GLOBAL_SHADER(FRootConstantsTestVS);
	SHADER_USE_PARAMETER_STRUCT(FRootConstantsTestVS, FRootConstantsTestShaderBase);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FRootConstantsTestVS, "/Plugin/RHITests/Private/TestRootConstants.usf", "RootConstantsMainVS", SF_Vertex);

class FRootConstantsTestPS : public FRootConstantsTestShaderBase
{
public:
	DECLARE_GLOBAL_SHADER(FRootConstantsTestPS);
	SHADER_USE_PARAMETER_STRUCT(FRootConstantsTestPS, FRootConstantsTestShaderBase);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWStructuredBuffer<uint4>, GraphicsOutputUAV)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FRootConstantsTestPS, "/Plugin/RHITests/Private/TestRootConstants.usf", "RootConstantsMainPS", SF_Pixel);

class FRootConstantsTestCS : public FRootConstantsTestShaderBase
{
public:
	DECLARE_GLOBAL_SHADER(FRootConstantsTestCS);
	SHADER_USE_PARAMETER_STRUCT(FRootConstantsTestCS, FRootConstantsTestShaderBase);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_UAV(RWStructuredBuffer<uint4>, OutputUAV)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FRootConstantsTestCS, "/Plugin/RHITests/Private/TestRootConstants.usf", "WriteRootConstants", SF_Compute);

// ---- Helpers ----

static bool ReadbackAndVerifyRootConstants(
	FRHICommandListImmediate& RHICmdList,
	FBufferRHIRef OutputBuffer,
	uint32 BufferSize,
	const FUint32Vector4& ExpectedValue,
	const TCHAR* TestName)
{
	FRHIGPUBufferReadback Readback(TestName);
	Readback.EnqueueCopy(RHICmdList, OutputBuffer, BufferSize);

	constexpr bool bFlushResources = true;
	RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

	const FUint32Vector4* OutputData = reinterpret_cast<const FUint32Vector4*>(Readback.Lock(BufferSize));

	bool bResult = true;
	if (*OutputData != ExpectedValue)
	{
		UE_LOG(LogRHIUnitTestCommandlet, Error,
			TEXT("Test failed. \"%s\"\n\nExpected: (%u, %u, %u, %u); Actual: (%u, %u, %u, %u)"),
			TestName,
			ExpectedValue.X, ExpectedValue.Y, ExpectedValue.Z, ExpectedValue.W,
			OutputData->X, OutputData->Y, OutputData->Z, OutputData->W);
		bResult = false;
	}

	Readback.Unlock();

	if (bResult)
	{
		UE_LOG(LogRHIUnitTestCommandlet, Display, TEXT("Test passed. \"%s\""), TestName);
	}

	return bResult;
}

// ---- Tests ----

// Draws a fullscreen triangle with a pixel shader that writes the graphics root constants
// to an output UAV.
bool RHIRootConstantsTests::Test_GraphicsShaderRootConstants(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHISupportsShaderRootConstants || !GRHIGlobals.SupportsPixelShaderUAVs)
	{
		return true;
	}

	const FUint32Vector4 GraphicsValue(1u, 2u, 3u, 4u);

	// Create output UAV
	const FRHIBufferCreateDesc OutputBufferDesc =
		FRHIBufferCreateDesc::CreateStructured<FUint32Vector4>(TEXT("GraphicsRootConstantsOutput"), 1)
		.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute);
	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(OutputBufferDesc);

	FUnorderedAccessViewRHIRef OutputUAV = RHICmdList.CreateUnorderedAccessView(
		OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(sizeof(FUint32Vector4)).SetNumElements(1)
	);

	// Create a 1x1 render target (we only need the pixel shader UAV write)
	const FRHITextureCreateDesc RTDesc =
		FRHITextureCreateDesc::Create2D(TEXT("RootConstantsTest_RT"), 1, 1, PF_B8G8R8A8)
		.SetFlags(ETextureCreateFlags::RenderTargetable)
		.SetInitialState(ERHIAccess::RTV);
	FTextureRHIRef RenderTarget = RHICmdList.CreateTexture(RTDesc);

	TShaderMapRef<FRootConstantsTestVS> VertexShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));
	TShaderMapRef<FRootConstantsTestPS> PixelShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

	FGraphicsPipelineStateInitializer GraphicsPSOInit;
	GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	GraphicsPSOInit.BoundShaderState.PixelShaderRHI  = PixelShader.GetPixelShader();
	GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
	GraphicsPSOInit.DepthStencilState  = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	GraphicsPSOInit.BlendState         = TStaticBlendState<>::GetRHI();
	GraphicsPSOInit.RasterizerState    = TStaticRasterizerState<>::GetRHI();
	GraphicsPSOInit.PrimitiveType      = EPrimitiveType::PT_TriangleList;

	RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVCompute, ERHIAccess::UAVGraphics));

	FRHITexture* ColorRTs[1] = { RenderTarget.GetReference() };
	FRHIRenderPassInfo RenderPassInfo(1, ColorRTs, ERenderTargetActions::DontLoad_DontStore);
	RHICmdList.BeginRenderPass(RenderPassInfo, TEXT("RootConstantsGraphicsTest"));
	{
		RHICmdList.SetViewport(0, 0, 0, 1, 1, 1);
		RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

		RHICmdList.SetShaderRootConstants(GraphicsValue);

		FRootConstantsTestPS::FParameters PSParams;
		PSParams.GraphicsOutputUAV = OutputUAV;

		FRHIBatchedShaderParameters& ShaderParams = RHICmdList.GetScratchShaderParameters();
		SetShaderParameters(ShaderParams, PixelShader, PSParams);
		RHICmdList.SetBatchedShaderParameters(PixelShader.GetPixelShader(), ShaderParams);

		RHICmdList.DrawPrimitive(0, 1, 1);
	}
	RHICmdList.EndRenderPass();

	RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVGraphics, ERHIAccess::CopySrc));

	return ReadbackAndVerifyRootConstants(RHICmdList, OutputBuffer, OutputBufferDesc.Size, GraphicsValue, TEXT("Graphics Shader Root Constants"));
}

// Dispatches a compute shader that writes root constants to a structured buffer.
bool RHIRootConstantsTests::Test_ComputeShaderRootConstants(FRHICommandListImmediate& RHICmdList)
{
	if (!GRHISupportsShaderRootConstants)
	{
		return true;
	}

	const FUint32Vector4 ComputeValue(11u, 22u, 33u, 44u);

	const FRHIBufferCreateDesc OutputBufferDesc =
		FRHIBufferCreateDesc::CreateStructured<FUint32Vector4>(TEXT("ComputeRootConstantsOutput"), 1)
		.AddUsage(EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute);
	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(OutputBufferDesc);

	FUnorderedAccessViewRHIRef OutputUAV = RHICmdList.CreateUnorderedAccessView(
		OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
			.SetType(FRHIViewDesc::EBufferType::Structured)
			.SetStride(sizeof(FUint32Vector4)).SetNumElements(1)
	);

	{
		TShaderMapRef<FRootConstantsTestCS> ComputeShader(GetGlobalShaderMap(GMaxRHIFeatureLevel));

		RHICmdList.SetComputeShaderRootConstants(ComputeValue);

		FRootConstantsTestCS::FParameters Parameters;
		Parameters.OutputUAV = OutputUAV;

		FComputeShaderUtils::Dispatch(RHICmdList, ComputeShader, Parameters, FIntVector(1, 1, 1));

		RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));
	}

	return ReadbackAndVerifyRootConstants(RHICmdList, OutputBuffer, OutputBufferDesc.Size, ComputeValue, TEXT("Compute Shader Root Constants"));
}
