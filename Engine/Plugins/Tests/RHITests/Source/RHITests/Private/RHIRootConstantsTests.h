// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FRHICommandListImmediate;

namespace RHIRootConstantsTests
{
	bool Test_GraphicsShaderRootConstants(FRHICommandListImmediate& RHICmdList);
	bool Test_ComputeShaderRootConstants(FRHICommandListImmediate& RHICmdList);
}
