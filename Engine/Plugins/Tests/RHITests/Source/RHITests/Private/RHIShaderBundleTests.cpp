// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHIShaderBundleTests.h"
#include "RHIBufferTests.h"
#include "CommonRenderResources.h"
#include "RHIStaticStates.h"
#include "ShaderCompilerCore.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "RHIShaderParameters.h"
#include "PipelineStateCache.h"

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FBundleTestParameters, )
	SHADER_PARAMETER(uint32, RecordSlot)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FBundleTestParameters, "BundleTest");

// ============================================================================
// Shaders
//
// CFLAG_ShaderBundle is required for all shaders used in a bundle.
//
// Current RHI limitations / workarounds (this needs to be fixed in the future):
//
// - Some RHIs require CFLAG_RootConstants for shader bundle dispatch.
//   We add it conditionally via RHISupportsShaderRootConstants().
//
// - LAYOUT_FIELD(TShaderUniformBufferParameter<T>, ...) does not
//   auto-register the UB in the compilation environment. We must
//   manually call FShaderUniformBufferParameter::ModifyCompilationEnvironment()
//   in each shader's MCE (guarded by WITH_EDITOR).
//   The LAYOUT_FIELD name must match the shader variable name from
//   IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT, or the generated HLSL
//   declaration will use the wrong variable name.
//
// - Some RHIs native bundle dispatch unconditionally dereferences
//   both Parameters_MSVS and Parameters_PS on graphics dispatches.
//   Both must be Emplace()'d even when a stage has no parameters.
//
// - GetGraphicsPipelineState() must be called after BeginRenderPass()
//   so the PSO cache's render target validation sees the active state.
// ============================================================================

class FBundleTestCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBundleTestCS);
	FBundleTestCS() = default;
	FBundleTestCS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		OutBuffer.Bind(Initializer.ParameterMap, TEXT("OutBuffer"), SPF_Mandatory);
		BundleTest.Bind(Initializer.ParameterMap, TEXT("BundleTest"));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ShaderBundle);
		if (RHISupportsShaderRootConstants(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
		}
#if WITH_EDITOR
		// Registers the BundleTest uniform buffer with the shader compilation environment:
		// adds its generated HLSL declaration to the virtual include path and populates the
		// ResourceTableMap/UniformBufferMap so the compiler produces a matching ParameterMap
		// entry. Without this, Bind() won't find the UB and it will silently remain unbound.
		FShaderUniformBufferParameter::ModifyCompilationEnvironment(TEXT("BundleTest"), *FBundleTestParameters::FTypeInfo::GetStructMetadata(), Parameters.Platform, OutEnvironment);
#endif
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return RHISupportsShaderBundleDispatch(Parameters.Platform);
	}

	LAYOUT_FIELD(FShaderResourceParameter, OutBuffer);
	LAYOUT_FIELD(TShaderUniformBufferParameter<FBundleTestParameters>, BundleTest);
};
IMPLEMENT_GLOBAL_SHADER(FBundleTestCS, "/Plugin/RHITests/Private/TestShaderBundle.usf", "BundleComputeMain", SF_Compute);

class FBundleTestMS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBundleTestMS);
	FBundleTestMS() = default;
	FBundleTestMS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		BundleTest.Bind(Initializer.ParameterMap, TEXT("BundleTest"));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ShaderBundle);
		if (RHISupportsShaderRootConstants(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
		}
		OutEnvironment.SetDefine(TEXT("SHADER_BUNDLE_TEST_MS"), 1);
#if WITH_EDITOR
		FShaderUniformBufferParameter::ModifyCompilationEnvironment(TEXT("BundleTest"), *FBundleTestParameters::FTypeInfo::GetStructMetadata(), Parameters.Platform, OutEnvironment);
#endif
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return RHISupportsShaderBundleDispatch(Parameters.Platform) && RHISupportsMeshShadersTier0(Parameters.Platform);
	}

	LAYOUT_FIELD(TShaderUniformBufferParameter<FBundleTestParameters>, BundleTest);
};
IMPLEMENT_GLOBAL_SHADER(FBundleTestMS, "/Plugin/RHITests/Private/TestShaderBundle.usf", "BundleMeshMain", SF_Mesh);

class FBundleTestPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBundleTestPS);
	FBundleTestPS() = default;
	FBundleTestPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		OutBuffer.Bind(Initializer.ParameterMap, TEXT("OutBuffer"), SPF_Mandatory);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ShaderBundle);
		if (RHISupportsShaderRootConstants(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
		}
		OutEnvironment.SetDefine(TEXT("SHADER_BUNDLE_TEST_PS"), 1);
#if WITH_EDITOR
		FShaderUniformBufferParameter::ModifyCompilationEnvironment(TEXT("BundleTest"), *FBundleTestParameters::FTypeInfo::GetStructMetadata(), Parameters.Platform, OutEnvironment);
#endif
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return RHISupportsShaderBundleDispatch(Parameters.Platform) && RHISupportsMeshShadersTier0(Parameters.Platform);
	}

	LAYOUT_FIELD(FShaderResourceParameter, OutBuffer);
};
IMPLEMENT_GLOBAL_SHADER(FBundleTestPS, "/Plugin/RHITests/Private/TestShaderBundle.usf", "BundlePixelMain", SF_Pixel);

class FBundleTestVS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBundleTestVS);
	FBundleTestVS() = default;
	FBundleTestVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ShaderBundle);
		if (RHISupportsShaderRootConstants(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
		}
		OutEnvironment.SetDefine(TEXT("SHADER_BUNDLE_TEST_VS"), 1);
#if WITH_EDITOR
		FShaderUniformBufferParameter::ModifyCompilationEnvironment(TEXT("BundleTest"), *FBundleTestParameters::FTypeInfo::GetStructMetadata(), Parameters.Platform, OutEnvironment);
#endif
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return RHISupportsShaderBundleDispatch(Parameters.Platform);
	}
};
IMPLEMENT_GLOBAL_SHADER(FBundleTestVS, "/Plugin/RHITests/Private/TestShaderBundle.usf", "BundleVertexMain", SF_Vertex);

class FBundleTestVSPS_PS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBundleTestVSPS_PS);
	FBundleTestVSPS_PS() = default;
	FBundleTestVSPS_PS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		OutBuffer.Bind(Initializer.ParameterMap, TEXT("OutBuffer"), SPF_Mandatory);
		BundleTest.Bind(Initializer.ParameterMap, TEXT("BundleTest"));
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_ShaderBundle);
		if (RHISupportsShaderRootConstants(Parameters.Platform))
		{
			OutEnvironment.CompilerFlags.Add(CFLAG_RootConstants);
		}
		OutEnvironment.SetDefine(TEXT("SHADER_BUNDLE_TEST_VSPS_PS"), 1);
#if WITH_EDITOR
		FShaderUniformBufferParameter::ModifyCompilationEnvironment(TEXT("BundleTest"), *FBundleTestParameters::FTypeInfo::GetStructMetadata(), Parameters.Platform, OutEnvironment);
#endif
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return RHISupportsShaderBundleDispatch(Parameters.Platform);
	}

	LAYOUT_FIELD(FShaderResourceParameter, OutBuffer);
	LAYOUT_FIELD(TShaderUniformBufferParameter<FBundleTestParameters>, BundleTest);
};
IMPLEMENT_GLOBAL_SHADER(FBundleTestVSPS_PS, "/Plugin/RHITests/Private/TestShaderBundle.usf", "BundleVSPSPixelMain", SF_Pixel);

static FUniformBufferRHIRef CreateBundleTest(uint32 RecordSlot)
{
	FBundleTestParameters Parameters;
	Parameters.RecordSlot = RecordSlot;
	return RHICreateUniformBuffer(&Parameters, &FBundleTestParameters::FTypeInfo::GetStructMetadata()->GetLayout(), UniformBuffer_MultiFrame);
}

// ============================================================================
// Test: Compute Shader Bundle
// ============================================================================

bool FRHIShaderBundleTests::Test_ComputeShaderBundle(FRHICommandListImmediate& RHICmdList)
{
	if (!RHISupportsShaderBundleDispatch(GMaxRHIShaderPlatform))
	{
		return true;
	}

	static constexpr uint32 NumRecords = 4;
	static constexpr uint32 OutputBufferStride = sizeof(uint32);
	static constexpr uint32 OutputBufferSize = OutputBufferStride * NumRecords;

	// Create output buffer
	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(
		FRHIBufferCreateDesc::Create(TEXT("BundleCS_Output"), OutputBufferSize, OutputBufferStride,
			EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute));

	FUnorderedAccessViewRHIRef OutputUAV = RHICmdList.CreateUnorderedAccessView(OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
		.SetType(FRHIViewDesc::EBufferType::Typed)
		.SetFormat(PF_R32_UINT));

	// Clear output
	RHICmdList.ClearUAVUint(OutputUAV, FUintVector4(0));

	// Create indirect args buffer: each record dispatches (1,1,1) groups
	// ArgStride = 12 bytes (3 uints: GroupX, GroupY, GroupZ)
	static constexpr uint32 ArgStride = 3 * sizeof(uint32);
	uint32 IndirectArgs[NumRecords * 3];
	for (uint32 i = 0; i < NumRecords; ++i)
	{
		IndirectArgs[i * 3 + 0] = 1; // GroupCountX
		IndirectArgs[i * 3 + 1] = 1; // GroupCountY
		IndirectArgs[i * 3 + 2] = 1; // GroupCountZ
	}

	FBufferRHIRef ArgBuffer = UE::RHIResourceUtils::CreateBufferFromArray(
		RHICmdList,
		TEXT("BundleCS_IndirectArgs"),
		EBufferUsageFlags::DrawIndirect | EBufferUsageFlags::UnorderedAccess,
		ERHIAccess::IndirectArgs,
		MakeConstArrayView(IndirectArgs));

	// Create the shader bundle
	FShaderBundleCreateInfo CreateInfo;
	CreateInfo.NumRecords = NumRecords;
	CreateInfo.ArgOffset = 0;
	CreateInfo.ArgStride = ArgStride;
	CreateInfo.Mode = ERHIShaderBundleMode::CS;
	FShaderBundleRHIRef ShaderBundle = RHICreateShaderBundle(CreateInfo);

	// Get the compute shader
	TShaderMapRef<FBundleTestCS> ComputeShader(GetGlobalShaderMap(GMaxRHIShaderPlatform));

	// Build dispatches
	FRHIBatchedShaderParametersAllocator& BundleAllocator = RHICmdList.GetScratchShaderParameters().Allocator;
	TArray<FRHIShaderBundleComputeDispatch> Dispatches;
	Dispatches.SetNum(NumRecords);

	TArray<FUniformBufferRHIRef> UniformBuffers;
	UniformBuffers.SetNum(NumRecords);

	for (uint32 i = 0; i < NumRecords; ++i)
	{
		FRHIShaderBundleComputeDispatch& Dispatch = Dispatches[i];
		Dispatch.RecordIndex = i;
		Dispatch.Shader = ComputeShader.GetComputeShader();
#if !PLATFORM_USE_FALLBACK_PSO
		// Some RHIs' native bundle dispatch requires pre-resolved pipeline state.
		Dispatch.RHIPipeline = RHICreateComputePipelineState(FComputePipelineStateInitializer(ComputeShader.GetComputeShader()));
#endif

		UniformBuffers[i] = CreateBundleTest(i);

		FRHIBatchedShaderParameters& Params = Dispatch.Parameters.Emplace(BundleAllocator);
		SetUAVParameter(Params, ComputeShader->OutBuffer, OutputUAV);
		SetUniformBufferParameter(Params, ComputeShader->BundleTest, UniformBuffers[i]);
		Params.Finish();
	}

	RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVCompute, ERHIAccess::UAVCompute, EResourceTransitionFlags::None));

	RHICmdList.DispatchComputeShaderBundle(
		ShaderBundle,
		ArgBuffer,
		TConstArrayView<FRHIShaderParameterResource>(),
		Dispatches,
		ERHIShaderBundleMethod::Native);

	RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVCompute, ERHIAccess::CopySrc, EResourceTransitionFlags::None));

	// Verify: each slot should contain slot+1
	uint32 Expected[NumRecords];
	for (uint32 i = 0; i < NumRecords; ++i)
	{
		Expected[i] = i + 1;
	}

	TConstArrayView<uint8> ExpectedView = MakeArrayView(reinterpret_cast<const uint8*>(Expected), sizeof(Expected));
	bool bSucceeded = FRHIBufferTests::VerifyBufferContents(TEXT("Test_ComputeShaderBundle"), RHICmdList, OutputBuffer, ExpectedView);

	return bSucceeded;
}

// ============================================================================
// Test: Graphics Shader Bundle (MSPS)
// ============================================================================

bool FRHIShaderBundleTests::Test_GraphicsShaderBundle_MSPS(FRHICommandListImmediate& RHICmdList)
{
	if (!RHISupportsShaderBundleDispatch(GMaxRHIShaderPlatform) || !GRHIGlobals.SupportsMeshShadersTier0)
	{
		return true;
	}

	static constexpr uint32 NumRecords = 4;
	static constexpr uint32 OutputBufferStride = sizeof(uint32);
	static constexpr uint32 OutputBufferSize = OutputBufferStride * NumRecords;
	static constexpr int32 RTWidth = 4;
	static constexpr int32 RTHeight = 4;

	// Create render target (required by render pass but we don't read it)
	FTextureRHIRef RenderTarget = RHICmdList.CreateTexture(
		FRHITextureCreateDesc(
			FRHITextureDesc(ETextureDimension::Texture2D, ETextureCreateFlags::RenderTargetable, PF_B8G8R8A8, FClearValueBinding(), FIntPoint(RTWidth, RTHeight), 1, 1, 1, 1, 0),
			ERHIAccess::RTV, TEXT("BundleMSPS_RT")));

	// Create output buffer (written by pixel shader)
	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(
		FRHIBufferCreateDesc::Create(TEXT("BundleMSPS_Output"), OutputBufferSize, OutputBufferStride,
			EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute));

	FUnorderedAccessViewRHIRef OutputUAV = RHICmdList.CreateUnorderedAccessView(OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
		.SetType(FRHIViewDesc::EBufferType::Typed)
		.SetFormat(PF_R32_UINT));

	RHICmdList.ClearUAVUint(OutputUAV, FUintVector4(0));

	// Indirect args: each record dispatches (1,1,1) mesh groups
	static constexpr uint32 ArgStride = 3 * sizeof(uint32);
	uint32 IndirectArgs[NumRecords * 3];
	for (uint32 i = 0; i < NumRecords; ++i)
	{
		IndirectArgs[i * 3 + 0] = 1; // GroupCountX
		IndirectArgs[i * 3 + 1] = 1; // GroupCountY
		IndirectArgs[i * 3 + 2] = 1; // GroupCountZ
	}

	FBufferRHIRef ArgBuffer = UE::RHIResourceUtils::CreateBufferFromArray(
		RHICmdList,
		TEXT("BundleMSPS_IndirectArgs"),
		EBufferUsageFlags::DrawIndirect | EBufferUsageFlags::UnorderedAccess,
		ERHIAccess::IndirectArgs,
		MakeConstArrayView(IndirectArgs));

	// Create the shader bundle
	FShaderBundleCreateInfo CreateInfo;
	CreateInfo.NumRecords = NumRecords;
	CreateInfo.ArgOffset = 0;
	CreateInfo.ArgStride = ArgStride;
	CreateInfo.Mode = ERHIShaderBundleMode::MSPS;
	FShaderBundleRHIRef ShaderBundle = RHICreateShaderBundle(CreateInfo);

	// Get shaders
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIShaderPlatform);
	TShaderMapRef<FBundleTestMS> MeshShader(ShaderMap);
	TShaderMapRef<FBundleTestPS> PixelShader(ShaderMap);

	// Build PSO initializer
	FGraphicsPipelineStateInitializer PSOInit;
	PSOInit.BoundShaderState.SetMeshShader(MeshShader.GetMeshShader());
	PSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	PSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	PSOInit.BlendState = TStaticBlendState<>::GetRHI();
	PSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	PSOInit.PrimitiveType = EPrimitiveType::PT_TriangleList;
	PSOInit.RenderTargetsEnabled = 1;
	PSOInit.RenderTargetFormats[0] = UE_PIXELFORMAT_TO_UINT8(PF_B8G8R8A8);

	// Transition output for graphics UAV access
	RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVCompute, ERHIAccess::UAVGraphics, EResourceTransitionFlags::None));

	// Begin render pass (must happen before GetGraphicsPipelineState for RT validation)
	// NOTE: GetGraphicsPipelineState must be called after BeginRenderPass so
	// that the PSO cache's render target validation sees the active RT state.
	FRHITexture* ColorRTs[1] = { RenderTarget.GetReference() };
	FRHIRenderPassInfo RenderPassInfo(1, ColorRTs, ERenderTargetActions::DontLoad_DontStore);
	RHICmdList.BeginRenderPass(RenderPassInfo, TEXT("Test_GraphicsShaderBundle_MSPS"));
	RHICmdList.SetViewport(0, 0, 0, float(RTWidth), float(RTHeight), 1);

	// Build dispatches
	FRHIBatchedShaderParametersAllocator& BundleAllocator = RHICmdList.GetScratchShaderParameters().Allocator;
	TArray<FRHIShaderBundleGraphicsDispatch> Dispatches;
	Dispatches.SetNum(NumRecords);

	TArray<FUniformBufferRHIRef> UniformBuffers;
	UniformBuffers.SetNum(NumRecords);

	for (uint32 i = 0; i < NumRecords; ++i)
	{
		FRHIShaderBundleGraphicsDispatch& Dispatch = Dispatches[i];
		Dispatch.RecordIndex = i;
		Dispatch.PipelineInitializer = PSOInit;
#if !PLATFORM_USE_FALLBACK_PSO
		Dispatch.PipelineState = GetGraphicsPipelineState(RHICmdList, PSOInit);
#endif

		UniformBuffers[i] = CreateBundleTest(i);

		// Mesh shader params (RecordSlot via uniform buffer)
		FRHIBatchedShaderParameters& MSParams = Dispatch.Parameters_MSVS.Emplace(BundleAllocator);
		SetUniformBufferParameter(MSParams, MeshShader->BundleTest, UniformBuffers[i]);
		MSParams.Finish();

		// Pixel shader params (OutBuffer UAV)
		FRHIBatchedShaderParameters& PSParams = Dispatch.Parameters_PS.Emplace(BundleAllocator);
		SetUAVParameter(PSParams, PixelShader->OutBuffer, OutputUAV);
		PSParams.Finish();
	}

	// Set up bundle state
	FRHIShaderBundleGraphicsState BundleState;
	BundleState.ViewRect = FIntRect(0, 0, RTWidth, RTHeight);
	BundleState.DepthMin = 0.0f;
	BundleState.DepthMax = 1.0f;
	BundleState.StencilRef = 0;

	RHICmdList.BeginUAVOverlap();

	RHICmdList.DispatchGraphicsShaderBundle(
		ShaderBundle,
		ArgBuffer,
		BundleState,
		TConstArrayView<FRHIShaderParameterResource>(),
		Dispatches,
		ERHIShaderBundleMethod::Native);

	RHICmdList.EndUAVOverlap();
	RHICmdList.EndRenderPass();

	// Readback and verify
	RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVGraphics, ERHIAccess::CopySrc, EResourceTransitionFlags::None));

	uint32 Expected[NumRecords];
	for (uint32 i = 0; i < NumRecords; ++i)
	{
		Expected[i] = i + 1;
	}

	TConstArrayView<uint8> ExpectedView = MakeArrayView(reinterpret_cast<const uint8*>(Expected), sizeof(Expected));
	bool bSucceeded = FRHIBufferTests::VerifyBufferContents(TEXT("Test_GraphicsShaderBundle_MSPS"), RHICmdList, OutputBuffer, ExpectedView);

	return bSucceeded;
}

// ============================================================================
// Test: Graphics Shader Bundle (VSPS)
// ============================================================================

bool FRHIShaderBundleTests::Test_GraphicsShaderBundle_VSPS(FRHICommandListImmediate& RHICmdList)
{
	if (!RHISupportsShaderBundleDispatch(GMaxRHIShaderPlatform))
	{
		return true;
	}

	static constexpr uint32 NumRecords = 4;
	static constexpr uint32 OutputBufferStride = sizeof(uint32);
	static constexpr uint32 OutputBufferSize = OutputBufferStride * NumRecords;
	static constexpr int32 RTWidth = 4;
	static constexpr int32 RTHeight = 4;

	// Create render target
	FTextureRHIRef RenderTarget = RHICmdList.CreateTexture(
		FRHITextureCreateDesc(
			FRHITextureDesc(ETextureDimension::Texture2D, ETextureCreateFlags::RenderTargetable, PF_B8G8R8A8, FClearValueBinding(), FIntPoint(RTWidth, RTHeight), 1, 1, 1, 1, 0),
			ERHIAccess::RTV, TEXT("BundleVSPS_RT")));

	// Create output buffer (written by pixel shader)
	FBufferRHIRef OutputBuffer = RHICmdList.CreateBuffer(
		FRHIBufferCreateDesc::Create(TEXT("BundleVSPS_Output"), OutputBufferSize, OutputBufferStride,
			EBufferUsageFlags::UnorderedAccess | EBufferUsageFlags::SourceCopy)
		.SetInitialState(ERHIAccess::UAVCompute));

	FUnorderedAccessViewRHIRef OutputUAV = RHICmdList.CreateUnorderedAccessView(OutputBuffer,
		FRHIViewDesc::CreateBufferUAV()
		.SetType(FRHIViewDesc::EBufferType::Typed)
		.SetFormat(PF_R32_UINT));

	RHICmdList.ClearUAVUint(OutputUAV, FUintVector4(0));

	// Indirect args for VSPS: {VertexCount, InstanceCount, FirstVertex, FirstInstance}
	// Each record draws 3 vertices (full-screen triangle), 1 instance
	static constexpr uint32 ArgStride = 4 * sizeof(uint32);
	uint32 IndirectArgs[NumRecords * 4];
	for (uint32 i = 0; i < NumRecords; ++i)
	{
		IndirectArgs[i * 4 + 0] = 3; // VertexCount
		IndirectArgs[i * 4 + 1] = 1; // InstanceCount
		IndirectArgs[i * 4 + 2] = 0; // FirstVertex
		IndirectArgs[i * 4 + 3] = 0; // FirstInstance
	}

	FBufferRHIRef ArgBuffer = UE::RHIResourceUtils::CreateBufferFromArray(
		RHICmdList,
		TEXT("BundleVSPS_IndirectArgs"),
		EBufferUsageFlags::DrawIndirect | EBufferUsageFlags::UnorderedAccess,
		ERHIAccess::IndirectArgs,
		MakeConstArrayView(IndirectArgs));

	// Create the shader bundle
	FShaderBundleCreateInfo CreateInfo;
	CreateInfo.NumRecords = NumRecords;
	CreateInfo.ArgOffset = 0;
	CreateInfo.ArgStride = ArgStride;
	CreateInfo.Mode = ERHIShaderBundleMode::VSPS;
	FShaderBundleRHIRef ShaderBundle = RHICreateShaderBundle(CreateInfo);

	// Get shaders
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIShaderPlatform);
	TShaderMapRef<FBundleTestVS> VertexShader(ShaderMap);
	TShaderMapRef<FBundleTestVSPS_PS> PixelShader(ShaderMap);

	// Build PSO initializer
	FGraphicsPipelineStateInitializer PSOInit;
	PSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
	PSOInit.BoundShaderState.VertexDeclarationRHI = GEmptyVertexDeclaration.VertexDeclarationRHI;
	PSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
	PSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
	PSOInit.BlendState = TStaticBlendState<>::GetRHI();
	PSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
	PSOInit.PrimitiveType = EPrimitiveType::PT_TriangleList;
	PSOInit.RenderTargetsEnabled = 1;
	PSOInit.RenderTargetFormats[0] = UE_PIXELFORMAT_TO_UINT8(PF_B8G8R8A8);

	// Transition output for graphics UAV access
	RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVCompute, ERHIAccess::UAVGraphics, EResourceTransitionFlags::None));

	// Begin render pass (must happen before GetGraphicsPipelineState for RT validation)
	FRHITexture* ColorRTs[1] = { RenderTarget.GetReference() };
	FRHIRenderPassInfo RenderPassInfo(1, ColorRTs, ERenderTargetActions::DontLoad_DontStore);
	RHICmdList.BeginRenderPass(RenderPassInfo, TEXT("Test_GraphicsShaderBundle_VSPS"));
	RHICmdList.SetViewport(0, 0, 0, float(RTWidth), float(RTHeight), 1);

	// Build dispatches
	FRHIBatchedShaderParametersAllocator& BundleAllocator = RHICmdList.GetScratchShaderParameters().Allocator;
	TArray<FRHIShaderBundleGraphicsDispatch> Dispatches;
	Dispatches.SetNum(NumRecords);

	TArray<FUniformBufferRHIRef> UniformBuffers;
	UniformBuffers.SetNum(NumRecords);

	for (uint32 i = 0; i < NumRecords; ++i)
	{
		FRHIShaderBundleGraphicsDispatch& Dispatch = Dispatches[i];
		Dispatch.RecordIndex = i;
		Dispatch.PipelineInitializer = PSOInit;
#if !PLATFORM_USE_FALLBACK_PSO
		Dispatch.PipelineState = GetGraphicsPipelineState(RHICmdList, PSOInit);
#endif

		UniformBuffers[i] = CreateBundleTest(i);

		// VS has no per-dispatch params (but must provide empty parameters for PS5 native bundles)
		// NOTE: Some RHIs' native bundle dispatch unconditionally dereferences
		// Parameters_MSVS, so it must be set even when the VS has no params.
		Dispatch.Parameters_MSVS.Emplace(BundleAllocator).Finish();

		// PS params: OutBuffer UAV + RecordSlot via uniform buffer
		FRHIBatchedShaderParameters& PSParams = Dispatch.Parameters_PS.Emplace(BundleAllocator);
		SetUAVParameter(PSParams, PixelShader->OutBuffer, OutputUAV);
		SetUniformBufferParameter(PSParams, PixelShader->BundleTest, UniformBuffers[i]);
		PSParams.Finish();
	}

	// Set up bundle state
	FRHIShaderBundleGraphicsState BundleState;
	BundleState.ViewRect = FIntRect(0, 0, RTWidth, RTHeight);
	BundleState.DepthMin = 0.0f;
	BundleState.DepthMax = 1.0f;
	BundleState.StencilRef = 0;

	RHICmdList.BeginUAVOverlap();

	RHICmdList.DispatchGraphicsShaderBundle(
		ShaderBundle,
		ArgBuffer,
		BundleState,
		TConstArrayView<FRHIShaderParameterResource>(),
		Dispatches,
		ERHIShaderBundleMethod::Native);

	RHICmdList.EndUAVOverlap();
	RHICmdList.EndRenderPass();

	// Readback and verify
	RHICmdList.Transition(FRHITransitionInfo(OutputUAV, ERHIAccess::UAVGraphics, ERHIAccess::CopySrc, EResourceTransitionFlags::None));

	uint32 Expected[NumRecords];
	for (uint32 i = 0; i < NumRecords; ++i)
	{
		Expected[i] = i + 1;
	}

	TConstArrayView<uint8> ExpectedView = MakeArrayView(reinterpret_cast<const uint8*>(Expected), sizeof(Expected));
	bool bSucceeded = FRHIBufferTests::VerifyBufferContents(TEXT("Test_GraphicsShaderBundle_VSPS"), RHICmdList, OutputBuffer, ExpectedView);

	return bSucceeded;
}
