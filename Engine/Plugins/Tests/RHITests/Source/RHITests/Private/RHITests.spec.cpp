// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "CoreMinimal.h"
#include "RenderUtils.h"
#include "RenderingThread.h"
#include "RHIBufferTests.h"
#include "RHITextureTests.h"
#include "RHIDrawTests.h"
#include "RHIClearTextureTests.h"
#include "RHIReadbackTests.h"
#include "RHIReservedResourceTests.h"
#include "RHIGraphicsUAVTests.h"
#include "RHIAllocatorTests.h"
#include "RHIBindlessTests.h"
#include "RHIRootConstantsTests.h"

BEGIN_DEFINE_SPEC(FAutomationRHITest, "Rendering.RHI", EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::NonNullRHI)
END_DEFINE_SPEC(FAutomationRHITest)
void FAutomationRHITest::Define()
{
	Describe("Test RHI Clear", [this]()
	{
		It("RHI Clear UINT VertexBuffer", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIBufferTests::Test_RHIClearUAVUint_VertexBuffer);
			TestEqual("Clear UINT VertexBuffer failed", bResult, 1);
		});

		It("RHI Clear Float VertexBuffer", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIBufferTests::Test_RHIClearUAVFloat_VertexBuffer);
			TestEqual("Clear Float VertexBuffer failed", bResult, 1);
		});

		It("RHI Clear UINT StructuredBuffer", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIBufferTests::Test_RHIClearUAVUint_StructuredBuffer);
			TestEqual("Clear UINT StructuredBuffer failed", bResult, 1);
		});

		It("RHI Clear Float StructuredBuffer", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIBufferTests::Test_RHIClearUAVFloat_StructuredBuffer);
			TestEqual("Clear Float StructuredBuffer failed", bResult, 1);
		});

		It("RHI Clear Texture2D", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHITextureTests::Test_RHIClearUAV_Texture2D);
			TestEqual("Clear Texture2D failed", bResult, 1);
		});

		It("RHI Clear Texture3D", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHITextureTests::Test_RHIClearUAV_Texture3D);
			TestEqual("Clear Texture3D failed", bResult, 1);
		});

		It("RHI Clear Render Targets", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIClearTextureTests::Test_ClearTexture);
			TestEqual("Clear Render Targets failed", bResult, 1);
		});
	});

	Describe("Test RHI Pixel Format", [this]()
	{
		It("RHI Target Formats", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHITextureTests::Test_RHIFormats);
			TestEqual("RHI Target Formats failed", bResult, 1);
		});
	});

	Describe("Test RHI Resource Update", [this]()
	{
		It("RHI Update Texture2D", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHITextureTests::Test_UpdateTexture);
			TestEqual("RHI Update Texture failed", bResult, 1);
		});
		
		It("RHI Multiple Lock Texture2D", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHITextureTests::Test_MultipleLockTexture2D);
			TestEqual("RHI Multiple Lock Texture2D failed", bResult, 1);
		});
	});

	Describe("Test RHI Copy", [this]
	{
		It("RHICopyTexture", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHITextureTests::Test_RHICopyTexture);
			TestEqual("RHICopyTexture", bResult, 1);
		});
	});

	Describe("Test RHI Readback", [this]
	{
		It("Buffer Readback", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIReadbackTests::Test_BufferReadback);
			TestEqual("BufferReadback", bResult, 1);
		});

		It("Texture Readback", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIReadbackTests::Test_TextureReadback);
			TestEqual("TextureReadback", bResult, 1);
		});
	});

	Describe("Test RHI Create Buffer", [this]
	{
		It("RHICreateBuffer", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIBufferTests::Test_RHICreateBuffer);
			TestEqual("Test_RHICreateBuffer", bResult, 1);
		});
	});

	Describe("Test RHI Create Buffer Parallel", [this]
	{
		It("RHICreateBuffer_Parallel", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIBufferTests::Test_RHICreateBuffer_Parallel);
			TestEqual("RHICreateBuffer_Parallel", bResult, 1);
		});
	});

	Describe("Test RHI Draw", [this]()
	{
		It("RHI DrawBaseVertexAndInstanceDirect", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIDrawTests::Test_DrawBaseVertexAndInstanceDirect);
			TestEqual("RHI DrawBaseVertexAndInstanceDirect", bResult, 1);
		});
		It("RHI DrawBaseVertexAndInstanceIndirect", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIDrawTests::Test_DrawBaseVertexAndInstanceIndirect);
			TestEqual("RHI DrawBaseVertexAndInstanceIndirect", bResult, 1);
		});
		It("RHI MultiDrawIndirect", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIDrawTests::Test_MultiDrawIndirect);
			TestEqual("RHI MultiDrawIndirect", bResult, 1);
		});
	});

	Describe("Test RHI Reserved Resource", [this]()
	{
		It("Create Reserved Volume Texture", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIReservedResourceTests::Test_ReservedResource_CreateVolumeTexture);
			TestEqual("Create Reserved Volume Texture failed", bResult, 1);
		});

		It("Create Reserved Texture", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIReservedResourceTests::Test_ReservedResource_CreateTexture);
			TestEqual("Create Reserved Texture failed", bResult, 1);
		});

		It("Create Reserved Texture With Mips", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIReservedResourceTests::Test_ReservedResource_CreateTextureWithMips);
			TestEqual("Create Reserved Texture With Mips failed", bResult, 1);
		});

		It("Create Reserved Buffer", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIReservedResourceTests::Test_ReservedResource_CreateBuffer);
			TestEqual("Create Reserved Buffer failed", bResult, 1);
		});

		It("Commit Reserved Buffer", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIReservedResourceTests::Test_ReservedResource_CommitBuffer);
			TestEqual("Commit Reserved Buffer failed", bResult, 1);
		});

		It("Decommit Reserved Buffer", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIReservedResourceTests::Test_ReservedResource_DecommitBuffer);
			TestEqual("Decommit Reserved Buffer failed", bResult, 1);
		});
	});


	Describe("Test RHI Graphics UAV Binding", [this]()
	{
		It("Pixel shader UAV", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIGraphicsUAVTests::Test_GraphicsUAV_PixelShader);
			TestEqual("Pixel shader UAV failed", bResult, 1);
		});

		It("Vertex shader UAV", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIGraphicsUAVTests::Test_GraphicsUAV_VertexShader);
			TestEqual("Vertex shader UAV failed", bResult, 1);
		});
	});

	Describe("Test RHI Allocators", [this]()
	{
		It("LockBuffer 16 Byte Alignment", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(FRHIAllocatorTests::Test_LockBuffer16ByteAlignment);
			TestEqual("LockBuffer 16 Byte Alignment Failed", bResult, 1);
		});
	});

	Describe("Test RHI Bindless", [this]()
	{
		It("RHI Resource Collection", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(RHIBindlessTests::Test_ResourceCollection);
			TestEqual("Resource Collection failed", bResult, 1);
		});
		It("RHI Descriptor Range SRV", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(RHIBindlessTests::Test_DescriptorRange_SRV);
			TestEqual("Descriptor Range SRV failed", bResult, 1);
		});
		It("RHI Descriptor Range Update With Offset", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(RHIBindlessTests::Test_DescriptorRange_UpdateWithOffset);
			TestEqual("Descriptor Range Update With Offset failed", bResult, 1);
		});
		It("RHI Descriptor Range UAV", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(RHIBindlessTests::Test_DescriptorRange_UAV);
			TestEqual("Descriptor Range UAV failed", bResult, 1);
		});
		It("RHI Descriptor Range SRV And UAV", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(RHIBindlessTests::Test_DescriptorRange_SRV_And_UAV);
			TestEqual("Descriptor Range SRV And UAV failed", bResult, 1);
		});
		It("RHI Descriptor Range Mixed Resource Types", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(RHIBindlessTests::Test_DescriptorRange_MixedResourceTypes);
			TestEqual("Descriptor Range Mixed Resource Types failed", bResult, 1);
		});
		It("RHI Bindless Accessible Uniform Buffer", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(RHIBindlessTests::Test_BindlessAccessibleUniformBuffer);
			TestEqual("Bindless Accessible Uniform Buffer failed", bResult, 1);
		});

	});

	Describe("Root Constants", [this]
	{
		It("RHI Graphics Shader Root Constants", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(RHIRootConstantsTests::Test_GraphicsShaderRootConstants);
			TestEqual("Graphics Shader Root Constants failed", bResult, 1);
		});

		It("RHI Compute Shader Root Constants", [this]()
		{
			bool bResult = RunOnRenderThreadSynchronous(RHIRootConstantsTests::Test_ComputeShaderRootConstants);
			TestEqual("Compute Shader Root Constants failed", bResult, 1);
		});
	});
}
