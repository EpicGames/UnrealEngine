// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHITestsCommon.h"

DEFINE_LOG_CATEGORY(LogRHIUnitTestCommandlet);

bool IsZeroMem(const void* Ptr, uint32 Size)
{
	uint8* Start = (uint8*)Ptr;
	uint8* End = Start + Size;
	while (Start < End)
	{
		if ((*Start++) != 0)
			return false;
	}

	return true;
}

bool RunOnRenderThreadSynchronous(TFunctionRef<bool(FRHICommandListImmediate&)> TestFunc)
{
	bool bResult = false;

	// Enqueue a single rendering command to hand control of the tests over to the rendering thread.
	ENQUEUE_RENDER_COMMAND(RunRHIUnitTestsCommand)([&bResult, TestFunc](FRHICommandListImmediate& RHICmdList)
	{
		bResult = TestFunc(RHICmdList);
	});

	// Flush to wait for the above rendering command to complete.
	FlushRenderingCommands();

	return bResult;
}
