// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHITestsModule.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h"
#include "RHIBufferTests.h"
#include "RHITextureTests.h"
#include "RHIDrawTests.h"
#include "RHIClearTextureTests.h"
#include "RHIReadbackTests.h"
#include "RHIReservedResourceTests.h"
#include "RHIGraphicsUAVTests.h"
#include "RHIAllocatorTests.h"
#include "RHIBindlessTests.h"
#include "RHIRootConstantsTests.h"
#include "RHIShaderBundleTests.h"

#define LOCTEXT_NAMESPACE "FRHITestsModule"

static FString GRHITestFilter;

bool ShouldRunRHITest(const TCHAR* Name)
{
	return GRHITestFilter.IsEmpty() || FCString::Stristr(Name, *GRHITestFilter);
}

static bool RunTests_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	bool bResult = true;

	// ------------------------------------------------
	// Reserved resources
	// ------------------------------------------------
	{

		RUN_TEST(FRHIReservedResourceTests::Test_ReservedResource_CommitBuffer(RHICmdList));
		RUN_TEST(FRHIReservedResourceTests::Test_ReservedResource_DecommitBuffer(RHICmdList));
		RUN_TEST(FRHIReservedResourceTests::Test_ReservedResource_CreateBuffer(RHICmdList));
		RUN_TEST(FRHIReservedResourceTests::Test_ReservedResource_CreateTexture(RHICmdList));
		RUN_TEST(FRHIReservedResourceTests::Test_ReservedResource_CreateTextureWithMips(RHICmdList));
		RUN_TEST(FRHIReservedResourceTests::Test_ReservedResource_CreateVolumeTexture(RHICmdList));
	}

	// ------------------------------------------------
	// Drawing
	// ------------------------------------------------
	{
		RUN_TEST(FRHIDrawTests::Test_DrawBaseVertexAndInstanceDirect(RHICmdList));
		RUN_TEST(FRHIDrawTests::Test_DrawBaseVertexAndInstanceIndirect(RHICmdList));
		RUN_TEST(FRHIDrawTests::Test_MultiDrawIndirect(RHICmdList));
	}

	// ------------------------------------------------
	// RHI Formats
	// ------------------------------------------------
	{
		RUN_TEST(FRHITextureTests::Test_RHIFormats(RHICmdList));
	}

	// ------------------------------------------------
	// RHIClearUAVUint / RHIClearUAVFloat tests
	// ------------------------------------------------
	{
		// Vertex/Structured Buffer
		RUN_TEST(FRHIBufferTests::Test_RHIClearUAVUint_VertexBuffer(RHICmdList));
		RUN_TEST(FRHIBufferTests::Test_RHIClearUAVFloat_VertexBuffer(RHICmdList));

		RUN_TEST(FRHIBufferTests::Test_RHIClearUAVUint_StructuredBuffer(RHICmdList));
		RUN_TEST(FRHIBufferTests::Test_RHIClearUAVFloat_StructuredBuffer(RHICmdList));

		// Texture2D/3D
		RUN_TEST(FRHITextureTests::Test_RHIClearUAV_Texture2D(RHICmdList));
		RUN_TEST(FRHITextureTests::Test_RHIClearUAV_Texture3D(RHICmdList));
	}

	// ------------------------------------------------
	// Texture Operations
	// ------------------------------------------------
	{
		RUN_TEST(FRHITextureTests::Test_RHICopyTexture(RHICmdList));
		RUN_TEST(FRHITextureTests::Test_UpdateTexture(RHICmdList));
		RUN_TEST(FRHITextureTests::Test_MultipleLockTexture2D(RHICmdList));
	}

	// ------------------------------------------------
	// Readback
	// ------------------------------------------------
	{
		RUN_TEST(FRHIReadbackTests::Test_BufferReadback(RHICmdList));
		RUN_TEST(FRHIReadbackTests::Test_TextureReadback(RHICmdList));
	}

	// ------------------------------------------------
	// Buffer Operations
	// ------------------------------------------------
	{
		RUN_TEST(FRHIBufferTests::Test_RHICreateBuffer(RHICmdList));
		RUN_TEST(FRHIBufferTests::Test_RHICreateBuffer_Parallel(RHICmdList));
	}

	// ------------------------------------------------
	// RT Operations
	// ------------------------------------------------
	{
		RUN_TEST(FRHIClearTextureTests::Test_ClearTexture(RHICmdList));
	}
	
	// ------------------------------------------------
	// Graphics UAV binding
	// ------------------------------------------------
	{
		RUN_TEST(FRHIGraphicsUAVTests::Test_GraphicsUAV_PixelShader(RHICmdList));
		RUN_TEST(FRHIGraphicsUAVTests::Test_GraphicsUAV_VertexShader(RHICmdList));
	}

	// ------------------------------------------------
	// RHI Allocators
	// ------------------------------------------------
	{
		RUN_TEST(FRHIAllocatorTests::Test_LockBuffer16ByteAlignment(RHICmdList));
	}

	// ------------------------------------------------
	// Bindless
	// ------------------------------------------------
	{
		RUN_TEST(RHIBindlessTests::Test_ResourceCollection(RHICmdList));
		RUN_TEST(RHIBindlessTests::Test_DescriptorRange_SRV(RHICmdList));
		RUN_TEST(RHIBindlessTests::Test_DescriptorRange_UpdateWithOffset(RHICmdList));
		RUN_TEST(RHIBindlessTests::Test_DescriptorRange_UAV(RHICmdList));
		RUN_TEST(RHIBindlessTests::Test_DescriptorRange_SRV_And_UAV(RHICmdList));
		RUN_TEST(RHIBindlessTests::Test_DescriptorRange_MixedResourceTypes(RHICmdList));
		RUN_TEST(RHIBindlessTests::Test_BindlessAccessibleUniformBuffer(RHICmdList));
	}

	// ------------------------------------------------
	// Root Constants
	// ------------------------------------------------
	{
		RUN_TEST(RHIRootConstantsTests::Test_GraphicsShaderRootConstants(RHICmdList));
		RUN_TEST(RHIRootConstantsTests::Test_ComputeShaderRootConstants(RHICmdList));
	}

	// ------------------------------------------------
	// Shader Bundles
	// ------------------------------------------------
	{
		RUN_TEST(FRHIShaderBundleTests::Test_ComputeShaderBundle(RHICmdList));
		RUN_TEST(FRHIShaderBundleTests::Test_GraphicsShaderBundle_MSPS(RHICmdList));
		RUN_TEST(FRHIShaderBundleTests::Test_GraphicsShaderBundle_VSPS(RHICmdList));
	}

	// @todo - add more tests
	return bResult;
}

void FRHITestsModule::StartupModule()
{
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("RHITests"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/RHITests"), PluginShaderDir);
}

void FRHITestsModule::RunAllTests()
{
	// -rhiunittest runs all tests; -rhiunittest=Filter runs only matching tests
	const TCHAR* CommandLine = FCommandLine::Get();
	GRHITestFilter.Empty();
	if (FParse::Param(CommandLine, TEXT("rhiunittest")) || FParse::Value(CommandLine, TEXT("rhiunittest="), GRHITestFilter))
	{
		if (!GRHITestFilter.IsEmpty())
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "RHI test filter: '%ls'", *GRHITestFilter);
		}
		if (RunOnRenderThreadSynchronous(RunTests_RenderThread))
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "RHI unit tests completed. All tests passed.");
		}
		else
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Error, "RHI unit tests completed. At least one test failed.");
		}
	}
}

void FRHITestsModule::ShutdownModule()
{

}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FRHITestsModule, RHITests)
