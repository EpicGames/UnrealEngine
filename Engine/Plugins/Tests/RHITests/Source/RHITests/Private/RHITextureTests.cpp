// Copyright Epic Games, Inc. All Rights Reserved.

#include "RHITextureTests.h"

bool FRHITextureTests::VerifyTextureContents(const TCHAR* TestName, FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, TFunctionRef<bool(void* Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 MipIndex, uint32 SliceIndex)> VerifyCallback)
{
	bool bResult = true;
	check(Texture->GetNumSamples() == 1); // @todo - support multi-sampled textures
	{
		FIntVector Size = Texture->GetSizeXYZ();

		for (uint32 MipIndex = 0; MipIndex < Texture->GetNumMips(); ++MipIndex)
		{
			uint32 MipWidth = FMath::Max(Size.X >> MipIndex, 1);
			uint32 MipHeight = FMath::Max(Size.Y >> MipIndex, 1);
			uint32 MipDepth = FMath::Max(Size.Z >> MipIndex, 1);

			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("FRHITextureTests_StagingTexture"), MipWidth, MipHeight, Texture->GetFormat())
				.SetFlags(ETextureCreateFlags::CPUReadback)
				.SetInitialState(ERHIAccess::CopyDest);

			for (uint32 Z = 0; Z < MipDepth; ++Z)
			{
				{
					FTextureRHIRef StagingTexture = RHICmdList.CreateTexture(Desc);

					FRHICopyTextureInfo CopyInfo = {};
					CopyInfo.Size = FIntVector(MipWidth, MipHeight, 1);
					CopyInfo.SourceMipIndex = MipIndex;
					if (Texture->GetTexture3D())
					{
						CopyInfo.SourceSliceIndex = 0;
						CopyInfo.SourcePosition.Z = Z;
					}
					else
					{
						CopyInfo.SourceSliceIndex = Z;
					}
					CopyInfo.NumSlices = 1;
					CopyInfo.NumMips = 1;

					RHICmdList.CopyTexture(Texture, StagingTexture, CopyInfo);

					RHICmdList.Transition(FRHITransitionInfo(StagingTexture, ERHIAccess::CopyDest, ERHIAccess::CPURead));

					FGPUFenceRHIRef GPUFence = RHICreateGPUFence(TEXT("ReadbackFence"));
					RHICmdList.WriteGPUFence(GPUFence);

					// @todo - refactor RHI readback API. This shouldn't be necessary
					constexpr bool bFlushResources = true;
					RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

					int32 Width, Height;
					void* Ptr;
					RHICmdList.MapStagingSurface(StagingTexture, GPUFence, Ptr, Width, Height);

					if (!VerifyCallback(Ptr, MipWidth, MipHeight, Width, Height, MipIndex, Z))
					{
						UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"%ls\" - Mip %d, Slice %d", TestName, MipIndex, Z);
						bResult = false;
					}

					RHICmdList.UnmapStagingSurface(StagingTexture);
				}
				RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
			}
		}
	}
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

	if (bResult)
	{
		UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"%ls\"", TestName);
	}

	return bResult;
}
