// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHITestsCommon.h"


class FRHIAllocatorTests
{
public:

	static bool Test_LockBuffer16ByteAlignment(FRHICommandListImmediate& RHICmdList);
};