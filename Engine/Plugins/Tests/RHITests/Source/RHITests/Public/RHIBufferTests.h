// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHITestsCommon.h"
#include "Async/ParallelFor.h"
#include "Math/RandomStream.h"

class FRHIBufferTests
{
public:

	// Copies data in the specified vertex buffer back to the CPU, and passes a pointer to that data to the provided verification lambda.
	static bool VerifyBufferContents(const TCHAR* TestName, FRHICommandListImmediate& RHICmdList, TArrayView<FRHIBuffer*> Buffers, TFunctionRef<bool(int32 BufferIndex, void* Ptr, uint32 NumBytes)> VerifyCallback);
	static bool VerifyBufferContents(const TCHAR* TestName, FRHICommandListImmediate& RHICmdList, FRHIBuffer* Buffer, TConstArrayView<uint8> ExpectedData);

private:

	template <typename TestLambdaType>
	static void ParallelDispatchCommands(FRHICommandListImmediate& RHICmdList, int32 NumTests, TestLambdaType TestLambda)
	{
		const int32 NumTasks = 32;

		int32 NumTestsPerTask = FMath::Max(NumTests / NumTasks, 1);
		int32 NumTestsLaunched = 0;

		TArray<FRHICommandListImmediate::FQueuedCommandList> CommandLists;

		while (NumTestsLaunched < NumTests)
		{
			FRHICommandList* RHICmdListUpload = new FRHICommandList();

			const int32 NumTestsInTask = FMath::Min(NumTestsPerTask, NumTests - NumTestsLaunched);

#if 1
			FFunctionGraphTask::CreateAndDispatchWhenReady(
				[TestLambda, RHICmdListUpload, NumTestsLaunched, NumTestsInTask](ENamedThreads::Type, const FGraphEventRef&)
			{
				FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);

				for (int32 Index = 0; Index < NumTestsInTask; ++Index)
				{
					TestLambda(*RHICmdListUpload, Index + NumTestsLaunched, NumTestsLaunched);
				}

				RHICmdListUpload->FinishRecording();

			}, TStatId(), nullptr, ENamedThreads::AnyHiPriThreadHiPriTask);

			CommandLists.Emplace(RHICmdListUpload);
#else
			for (int32 Index = 0; Index < NumTestsInTask; ++Index)
			{
				TestLambda(*RHICmdListUpload, Index + NumTestsLaunched, NumTestsLaunched);
			}
			delete RHICmdListUpload;
#endif

			NumTestsLaunched += NumTestsInTask;
		}

		RHICmdList.QueueAsyncCommandListSubmit(CommandLists);
		RHICmdList.ImmediateFlush(EImmediateFlushType::WaitForOutstandingTasksOnly);
	}

	template <typename BufferType, typename ValueType, uint32 NumTestBytes>
	static bool RunTest_UAVClear_Buffer(
		FRHICommandListImmediate& RHICmdList
		, const FString& TestName
		, BufferType* BufferRHI
		, FRHIUnorderedAccessView* WholeResourceUAV
		, FRHIUnorderedAccessView* OffsetUAV
		, const ValueType& ClearValue
		, void(FRHIComputeCommandList::* ClearPtr)(FRHIUnorderedAccessView*, ValueType const&)
		, const uint8(&TestValue)[NumTestBytes]
	)
	{
		auto const InfoWhole = WholeResourceUAV->GetDesc().Buffer.UAV.GetViewInfo(BufferRHI);
		auto const InfoOffset = OffsetUAV->GetDesc().Buffer.UAV.GetViewInfo(BufferRHI);

		uint32 BufferSize = BufferRHI->GetDesc().Size;

		checkf(BufferSize % NumTestBytes == 0, TEXT("BufferSize must be a multiple of NumTestBytes."));

		// Build arrays of the expected buffer values
		TArray<uint8> Expected_Zero;
		TArray<uint8> Expected_Cleared;
		TArray<uint8> Expected_Offset;
		{
			Expected_Zero.SetNumZeroed(BufferSize);
			Expected_Cleared.SetNumUninitialized(BufferSize);
			for (uint32 Index = 0; Index < BufferSize; Index += NumTestBytes)
			{
				FMemory::Memcpy(&Expected_Cleared[Index], TestValue, NumTestBytes);
			}

			Expected_Offset = Expected_Cleared;
			for (uint32 Element = 0; Element < InfoOffset.NumElements; Element++)
			{
				FMemory::Memzero(&Expected_Offset[(Element * InfoOffset.StrideInBytes) + InfoOffset.OffsetInBytes], InfoOffset.StrideInBytes);
			}
		}

		// Test clear buffer to zero
		RHICmdList.Transition(FRHITransitionInfo(WholeResourceUAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

		ValueType ZerosValue;
		FMemory::Memset(&ZerosValue, 0, sizeof(ZerosValue));
		(RHICmdList.*ClearPtr)(WholeResourceUAV, ZerosValue);

		RHICmdList.Transition(FRHITransitionInfo(WholeResourceUAV, ERHIAccess::Unknown, ERHIAccess::CopySrc));
		bool bResult0 = VerifyBufferContents(*FString::Printf(TEXT("%s - clear whole resource to zero"), *TestName), RHICmdList, BufferRHI, Expected_Zero);

		FString ClearValueStr;
		if constexpr (std::is_same_v<ValueType, FVector4f>)
		{
			ClearValueStr = FString::Printf(TEXT("%f %f %f %f"), ClearValue.X, ClearValue.Y, ClearValue.Z, ClearValue.W);
		}
		else
		{
			ClearValueStr = FString::Printf(TEXT("0x%08x 0x%08x 0x%08x 0x%08x"), ClearValue.X, ClearValue.Y, ClearValue.Z, ClearValue.W);
		}

		// Clear the buffer to the provided value
		RHICmdList.Transition(FRHITransitionInfo(WholeResourceUAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
		(RHICmdList.*ClearPtr)(WholeResourceUAV, ClearValue);
		RHICmdList.Transition(FRHITransitionInfo(WholeResourceUAV, ERHIAccess::Unknown, ERHIAccess::CopySrc));
		bool bResult1 = VerifyBufferContents(*FString::Printf(TEXT("%s - clear to (%s)"), *TestName, *ClearValueStr), RHICmdList, BufferRHI, Expected_Cleared);

		// Clear the offset UAV to zero, leaving the outer elements untouched.
		RHICmdList.Transition(FRHITransitionInfo(OffsetUAV, ERHIAccess::Unknown, ERHIAccess::UAVCompute));
		(RHICmdList.*ClearPtr)(OffsetUAV, ZerosValue);
		RHICmdList.Transition(FRHITransitionInfo(OffsetUAV, ERHIAccess::Unknown, ERHIAccess::CopySrc));
		bool bResult2 = VerifyBufferContents(*FString::Printf(TEXT("%s - clear offset UAV to zero"), *TestName), RHICmdList, BufferRHI, Expected_Offset);

		return bResult0 && bResult1 && bResult2;
	}

	template <typename ValueType, uint32 NumTestBytes>
	static bool RunTest_UAVClear_Buffer(
		  FRHICommandListImmediate& RHICmdList
		, EBufferUsageFlags BufferFlags
		, uint32 BufferSize
		, EPixelFormat Format
		, uint32 Stride
		, bool bRawView
		, const ValueType& ClearValue
		, void(FRHIComputeCommandList::* ClearPtr)(FRHIUnorderedAccessView*, ValueType const&)
		, const uint8(&TestValue)[NumTestBytes]
	)
	{
		checkf((Format == PF_Unknown) ^ (Stride == 0), TEXT("Either format or stride should be provided, not both."));

		bool bStructured = Format == PF_Unknown;
		bool bRaw = EnumHasAnyFlags(BufferFlags, BUF_ByteAddressBuffer);

		FString TestName = bStructured
			? FString::Printf(TEXT("RunTest_UAVClear_Buffer (Structured) - Stride: %d"), Stride)
			: FString::Printf(TEXT("RunTest_UAVClear_Buffer (%s) - Format: %s, Raw View: %s")
				, bRaw ? TEXT("Raw") : TEXT("Typed")
				, GPixelFormats[Format].Name
				, bRawView ? TEXT("True") : TEXT("False")
			);

		if (!bStructured && !GPixelFormats[Format].Supported)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test skipped. \"%ls\". Unsupported format.", *TestName);
			return true;
		}

		const FRHIBufferCreateDesc Desc =
			FRHIBufferCreateDesc::Create(*TestName, BufferSize, Stride, BufferFlags | BUF_ShaderResource | BUF_UnorderedAccess | BUF_SourceCopy)
			.SetInitialState(ERHIAccess::UAVMask);

		FBufferRHIRef Buffer = RHICmdList.CreateBuffer(Desc);

		auto UAVCreateDesc = FRHIViewDesc::CreateBufferUAV();
		auto SRVCreateDesc = FRHIViewDesc::CreateBufferSRV();

		if (bRawView)
		{
			UAVCreateDesc.SetType(FRHIViewDesc::EBufferType::Raw);
			SRVCreateDesc.SetType(FRHIViewDesc::EBufferType::Raw);
		}
		else
		{
			UAVCreateDesc
				.SetType(bStructured ? FRHIViewDesc::EBufferType::Structured : FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format)
				.SetStride(Stride);

			SRVCreateDesc
				.SetType(bStructured ? FRHIViewDesc::EBufferType::Structured : FRHIViewDesc::EBufferType::Typed)
				.SetFormat(Format)
				.SetStride(Stride);
		}

		FUnorderedAccessViewRHIRef WholeResourceUAV = RHICmdList.CreateUnorderedAccessView(Buffer, UAVCreateDesc);
		auto const ViewInfo = WholeResourceUAV->GetDesc().Buffer.UAV.GetViewInfo(Buffer);

		// Use an offset of 1 element (or minimum supported alignment)
        uint32 TestOffset = ViewInfo.StrideInBytes;
        if (bRawView)
        {
            TestOffset = FMath::Max(TestOffset, 16u);
        }
        if (!bStructured)
        {
			checkf(uint32(RHIGetMinimumAlignmentForBufferBackedSRV(Format)) < BufferSize, TEXT("Minimum aligned buffer size is %lld, buffer size is %d, PixelFormat: %d"), RHIGetMinimumAlignmentForBufferBackedSRV(Format), BufferSize, (uint32_t)Format);
			
            TestOffset = FMath::Max(TestOffset, uint32(RHIGetMinimumAlignmentForBufferBackedSRV(Format)));
        }

		// Leave up to 5 elements at the end of the buffer
		uint32 NumElements = FMath::DivideAndRoundDown(ViewInfo.SizeInBytes - TestOffset, ViewInfo.StrideInBytes);
		NumElements = FMath::Max(1, int32(NumElements) - 5);

		// Adjust the UAV desc so that it covers a region in the middle of the buffer
		UAVCreateDesc
			.SetOffsetInBytes(TestOffset)
			.SetNumElements(NumElements);

		FUnorderedAccessViewRHIRef OffsetUAV = RHICmdList.CreateUnorderedAccessView(Buffer, UAVCreateDesc);

		// Also test SRV creation
		FShaderResourceViewRHIRef SRV = RHICmdList.CreateShaderResourceView(Buffer, SRVCreateDesc);

		bool bResult = RunTest_UAVClear_Buffer(RHICmdList, TestName, Buffer.GetReference(), WholeResourceUAV, OffsetUAV, ClearValue, ClearPtr, TestValue);

		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

		return bResult;
	}

public:
	static bool Test_RHIClearUAVUint_VertexBuffer(FRHICommandListImmediate& RHICmdList)
	{
		bool bResult = true;

		{
			// Unsigned int tests (values within range of underlying format, so no conversion should happen)
			const FUintVector4 ClearValueUint8(0x01, 0x23, 0x45, 0x67);
			const FUintVector4 ClearValueUint16(0x0123, 0x4567, 0x89ab, 0xcdef);
			const FUintVector4 ClearValueUint32(0x01234567, 0x89abcdef, 0x8899aabb, 0xccddeeff);

			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R8_UINT          , 0, false, ClearValueUint8 , &FRHICommandListImmediate::ClearUAVUint, { 0x01 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R8G8B8A8_UINT    , 0, false, ClearValueUint8 , &FRHICommandListImmediate::ClearUAVUint, { 0x01, 0x23, 0x45, 0x67 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16_UINT         , 0, false, ClearValueUint16, &FRHICommandListImmediate::ClearUAVUint, { 0x23, 0x01 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16G16_UINT      , 0, false, ClearValueUint16, &FRHICommandListImmediate::ClearUAVUint, { 0x23, 0x01, 0x67, 0x45 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16G16B16A16_UINT, 0, false, ClearValueUint16, &FRHICommandListImmediate::ClearUAVUint, { 0x23, 0x01, 0x67, 0x45, 0xab, 0x89, 0xef, 0xcd }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R32_UINT         , 0, false, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R32G32_UINT      , 0, false, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01, 0xef, 0xcd, 0xab, 0x89 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R32G32B32A32_UINT, 0, false, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01, 0xef, 0xcd, 0xab, 0x89, 0xbb, 0xaa, 0x99, 0x88, 0xff, 0xee, 0xdd, 0xcc }));

			// Signed integer
			const FUintVector4 ClearValueInt16_Positive(0x1122, 0x3344, 0x5566, 0x7788);
			const FUintVector4 ClearValueInt32_Positive(0x10112233, 0x44556677, 0x0899aabb, 0x4cddeeff);
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16_SINT         , 0, false, ClearValueInt16_Positive, &FRHICommandListImmediate::ClearUAVUint, { 0x22, 0x11 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16G16B16A16_SINT, 0, false, ClearValueInt16_Positive, &FRHICommandListImmediate::ClearUAVUint, { 0x22, 0x11, 0x44, 0x33, 0x66, 0x55, 0x88, 0x77 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R32_SINT         , 0, false, ClearValueInt32_Positive, &FRHICommandListImmediate::ClearUAVUint, { 0x33, 0x22, 0x11, 0x10 }));

			const FUintVector4 ClearValueInt16_Negative(0xffff9122, 0xffffb344, 0xffffd566, 0xfffff788);
			const FUintVector4 ClearValueInt32_Negative(0x80112233, 0xc4556677, 0x8899aabb, 0xccddeeff);
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16_SINT         , 0, false, ClearValueInt16_Negative, &FRHICommandListImmediate::ClearUAVUint, { 0x22, 0x91 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16G16B16A16_SINT, 0, false, ClearValueInt16_Negative, &FRHICommandListImmediate::ClearUAVUint, { 0x22, 0x91, 0x44, 0xb3, 0x66, 0xd5, 0x88, 0xf7 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R32_SINT         , 0, false, ClearValueInt32_Negative, &FRHICommandListImmediate::ClearUAVUint, { 0x33, 0x22, 0x11, 0x80 }));

			// Raw (ByteAddress) buffer. A raw view uses only the first channle of the clear value, directly as a 32bit integer. Typed views still do the appropriate format conversion.
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R8G8B8A8_UINT, 0, false, ClearValueUint8 , &FRHICommandListImmediate::ClearUAVUint, { 0x01, 0x23, 0x45, 0x67 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R8G8B8A8_UINT, 0, true , ClearValueUint8 , &FRHICommandListImmediate::ClearUAVUint, { 0x01, 0x00, 0x00, 0x00 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R16G16_UINT  , 0, false, ClearValueUint16, &FRHICommandListImmediate::ClearUAVUint, { 0x23, 0x01, 0x67, 0x45 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R16G16_UINT  , 0, true , ClearValueUint16, &FRHICommandListImmediate::ClearUAVUint, { 0x23, 0x01, 0x00, 0x00 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R32_UINT     , 0, false, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R32_UINT     , 0, true , ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01 }));
		}

		{
			// Clamping unsigned int tests (components of ClearValueUint are > 0xffff, so will be clamped by the format conversion for formats < 32 bits per channel wide).
			const FUintVector4 ClearValueUint(0xeeffccdd, 0xaabb8899, 0x66774455, 0x22330011);
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R8_UINT          , 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xff }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16_UINT         , 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xff, 0xff }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16G16_UINT      , 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xff, 0xff, 0xff, 0xff }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16G16B16A16_UINT, 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff, 0xff }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R8G8B8A8_UINT    , 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xff, 0xff, 0xff, 0xff }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R32_UINT         , 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xdd, 0xcc, 0xff, 0xee }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R32G32_UINT      , 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xdd, 0xcc, 0xff, 0xee, 0x99, 0x88, 0xbb, 0xaa }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R32G32B32A32_UINT, 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xdd, 0xcc, 0xff, 0xee, 0x99, 0x88, 0xbb, 0xaa, 0x55, 0x44, 0x77, 0x66, 0x11, 0x00, 0x33, 0x22 }));

			// Signed integer
			const FUintVector4 ClearValueInt16_ClampToMaxInt16(0x8001, 0x8233, 0x8455, 0x8677);
			const FUintVector4 ClearValueInt16_ClampToMinInt16(0xfabc7123, 0x80123456, 0x80203040, 0x8a0b0c0d);
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16_SINT         , 0, false, ClearValueInt16_ClampToMaxInt16, &FRHICommandListImmediate::ClearUAVUint, { 0xff, 0x7f }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16_SINT         , 0, false, ClearValueInt16_ClampToMinInt16, &FRHICommandListImmediate::ClearUAVUint, { 0x00, 0x80 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16G16B16A16_SINT, 0, false, ClearValueInt16_ClampToMaxInt16, &FRHICommandListImmediate::ClearUAVUint, { 0xff, 0x7f, 0xff, 0x7f, 0xff, 0x7f, 0xff, 0x7f }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16G16B16A16_SINT, 0, false, ClearValueInt16_ClampToMinInt16, &FRHICommandListImmediate::ClearUAVUint, { 0x00, 0x80, 0x00, 0x80, 0x00, 0x80, 0x00, 0x80 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R32_SINT         , 0, false, ClearValueUint                 , &FRHICommandListImmediate::ClearUAVUint, { 0xdd, 0xcc, 0xff, 0xee }));

			// Raw (ByteAddress) buffer. A raw view uses only the first channle of the clear value, directly as a 32bit integer. Typed views still do the appropriate format conversion.
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R16G16_UINT  , 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xff, 0xff, 0xff, 0xff }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R16G16_UINT  , 0, true , ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xdd, 0xcc, 0xff, 0xee }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R8G8B8A8_UINT, 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xff, 0xff, 0xff, 0xff }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R8G8B8A8_UINT, 0, true , ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xdd, 0xcc, 0xff, 0xee }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R32_UINT     , 0, false, ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xdd, 0xcc, 0xff, 0xee }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R32_UINT     , 0, true , ClearValueUint, &FRHICommandListImmediate::ClearUAVUint, { 0xdd, 0xcc, 0xff, 0xee }));
		}

		return bResult;
	}

	static bool Test_RHIClearUAVFloat_VertexBuffer(FRHICommandListImmediate& RHICmdList)
	{
		bool bResult = true;

		{
			// Float       32-bit     16-bit
			// 0.2345  = 0x3e7020c5 | 0x3381
			// 0.8499  = 0x3f59930c | 0x3acc
			// 0.00145 = 0x3abe0ded | 0x15f0
			// 0.417   = 0x3ed58106 | 0x36ac
			const FVector4f ClearValueFloat(0.2345f, 0.8499f, 0.417f, 0.00145f);

			// Half precision float tests
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16F          , 0, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0x81, 0x33 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R16F_FILTER   , 0, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0x81, 0x33 }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_G16R16F       , 0, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0x81, 0x33, 0xcc, 0x3a }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_G16R16F_FILTER, 0, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0x81, 0x33, 0xcc, 0x3a }));

			// Full precision float tests
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_R32_FLOAT    , 0, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_G32R32F      , 0, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e, 0x0c, 0x93, 0x59, 0x3f }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer, 2048, PF_A32B32G32R32F, 0, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e, 0x0c, 0x93, 0x59, 0x3f, 0x06, 0x81, 0xd5, 0x3e, 0xed, 0x0d, 0xbe, 0x3a }));

			// @todo - 11,11,10 formats etc.

			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_G16R16F       , 0, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0x81, 0x33, 0xcc, 0x3a }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_G16R16F       , 0, true , ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_G16R16F_FILTER, 0, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0x81, 0x33, 0xcc, 0x3a }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_G16R16F_FILTER, 0, true , ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R32_FLOAT     , 0, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_VertexBuffer | BUF_ByteAddressBuffer, 2048, PF_R32_FLOAT     , 0, true , ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e }));
		}

		return bResult;
	}

	static bool Test_RHIClearUAVUint_StructuredBuffer(FRHICommandListImmediate& RHICmdList)
	{
		bool bResult = true;

		// Structured buffer clears should memset the whole resource to (uint32)ClearValue.X, ignoring other channels.
		const FUintVector4 ClearValueUint8(0x01, 0x23, 0x45, 0x67);
		const FUintVector4 ClearValueUint16(0x0123, 0x4567, 0x89ab, 0xcdef);
		const FUintVector4 ClearValueUint32(0x01234567, 0x89abcdef, 0x8899aabb, 0xccddeeff);

		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown,  4, false, ClearValueUint8 , &FRHICommandListImmediate::ClearUAVUint, { 0x01, 0x00, 0x00, 0x00 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown,  8, false, ClearValueUint8 , &FRHICommandListImmediate::ClearUAVUint, { 0x01, 0x00, 0x00, 0x00, 0x01, 0x00, 0x00, 0x00 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 264, PF_Unknown, 12, false, ClearValueUint8 , &FRHICommandListImmediate::ClearUAVUint, { 0x01, 0x00, 0x00, 0x00, 0x01, 0x00, 0x00, 0x00, 0x01, 0x00, 0x00, 0x00 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown, 16, false, ClearValueUint8 , &FRHICommandListImmediate::ClearUAVUint, { 0x01, 0x00, 0x00, 0x00, 0x01, 0x00, 0x00, 0x00, 0x01, 0x00, 0x00, 0x00, 0x01, 0x00, 0x00, 0x00 }));

		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown,  4, false, ClearValueUint16, &FRHICommandListImmediate::ClearUAVUint, { 0x23, 0x01, 0x00, 0x00 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown,  8, false, ClearValueUint16, &FRHICommandListImmediate::ClearUAVUint, { 0x23, 0x01, 0x00, 0x00, 0x23, 0x01, 0x00, 0x00 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 264, PF_Unknown, 12, false, ClearValueUint16, &FRHICommandListImmediate::ClearUAVUint, { 0x23, 0x01, 0x00, 0x00, 0x23, 0x01, 0x00, 0x00, 0x23, 0x01, 0x00, 0x00 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown, 16, false, ClearValueUint16, &FRHICommandListImmediate::ClearUAVUint, { 0x23, 0x01, 0x00, 0x00, 0x23, 0x01, 0x00, 0x00, 0x23, 0x01, 0x00, 0x00, 0x23, 0x01, 0x00, 0x00 }));

		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown,  4, false, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown,  8, false, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 264, PF_Unknown, 12, false, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown, 16, false, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01 }));

		// Large stride
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown, 32, false, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint,
		{
			0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01,
			0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01, 0x67, 0x45, 0x23, 0x01
		}));

		// Signed integer
		const FUintVector4 ClearValueInt32_Negative(0x80112233, 0xc4556677, 0x8899aabb, 0xccddeeff);
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown,  4, false, ClearValueInt32_Negative, &FRHICommandListImmediate::ClearUAVUint, { 0x33, 0x22, 0x11, 0x80 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown,  8, false, ClearValueInt32_Negative, &FRHICommandListImmediate::ClearUAVUint, { 0x33, 0x22, 0x11, 0x80, 0x33, 0x22, 0x11, 0x80 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 264, PF_Unknown, 12, false, ClearValueInt32_Negative, &FRHICommandListImmediate::ClearUAVUint, { 0x33, 0x22, 0x11, 0x80, 0x33, 0x22, 0x11, 0x80, 0x33, 0x22, 0x11, 0x80 }));
		RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown, 16, false, ClearValueInt32_Negative, &FRHICommandListImmediate::ClearUAVUint, { 0x33, 0x22, 0x11, 0x80, 0x33, 0x22, 0x11, 0x80, 0x33, 0x22, 0x11, 0x80, 0x33, 0x22, 0x11, 0x80 }));

		return bResult;
	}

	static bool Test_RHIClearUAVFloat_StructuredBuffer(FRHICommandListImmediate& RHICmdList)
	{
		bool bResult = true;

		{
			// Float       32-bit  
			// 0.2345  = 0x3e7020c5
			// 0.8499  = 0x3f59930c
			// 0.00145 = 0x3abe0ded
			// 0.417   = 0x3ed58106
			const FVector4f ClearValueFloat(0.2345f, 0.8499f, 0.417f, 0.00145f);

			// Full precision float tests
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown,  4, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown,  8, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e, 0xc5, 0x20, 0x70, 0x3e }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 264, PF_Unknown, 12, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e, 0xc5, 0x20, 0x70, 0x3e, 0xc5, 0x20, 0x70, 0x3e }));
			RUN_TEST(RunTest_UAVClear_Buffer(RHICmdList, BUF_StructuredBuffer, 256, PF_Unknown, 16, false, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0xc5, 0x20, 0x70, 0x3e, 0xc5, 0x20, 0x70, 0x3e, 0xc5, 0x20, 0x70, 0x3e, 0xc5, 0x20, 0x70, 0x3e }));
		}

		return bResult;
	}

	static bool Test_RHICreateBuffer_Buffer(FRHICommandListImmediate& RHICmdList, uint32 BufferSize)
	{
		SCOPED_NAMED_EVENT_TEXT("Test_RHICreateBuffer", FColor::Magenta);

		FString TestName = FString::Printf(TEXT("Test_RHICreateBuffer Size: %d"), BufferSize);

		if (BufferSize == 0)
		{
			FBufferRHIRef BufferRHI = RHICmdList.CreateBuffer(FRHIBufferCreateDesc::CreateNull(*TestName).SetInitialState(ERHIAccess::VertexOrIndexBuffer));
			return BufferRHI != nullptr;
		}

		// Build arrays of the expected buffer values
		TArray<uint8> Expected_Zero;
		TArray<uint8> Expected_Offsets;
		{
			Expected_Zero.SetNumZeroed(BufferSize);
			Expected_Offsets.SetNumUninitialized(BufferSize);
			for (uint32 Index = 0; Index < BufferSize; Index++)
			{
				// Rollover is expected here
				Expected_Offsets[Index] = static_cast<uint8>(Index);
			}
		}

		const FRHIBufferCreateDesc CommonCreateDesc =
			FRHIBufferCreateDesc::Create(*TestName, BufferSize, 0, EBufferUsageFlags::VertexBuffer)
			// Required for verifying buffer contents
			.AddUsage(EBufferUsageFlags::SourceCopy)
			.SetInitialState(ERHIAccess::CopySrc);

		if (false) // check Fatal conditions
		{
			FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(CommonCreateDesc);
			RHICmdList.ImmediateFlush(EImmediateFlushType::DispatchToRHIThread);

			FBufferRHIRef BufferRHI = Initializer.Finalize();
		}

		bool bResult = true;

		{
			// Create Zeroed
			FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc(CommonCreateDesc)
				.SetInitActionZeroData();

			FBufferRHIRef BufferRHI = RHICmdList.CreateBuffer(CreateDesc);

			const bool bDataMatches = VerifyBufferContents(*FString::Printf(TEXT("%s - ZeroData"), *TestName), RHICmdList, BufferRHI, Expected_Zero);
			bResult = bDataMatches ? bResult : false;
		}

		{
			// Create using initializer
			FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc(CommonCreateDesc)
				.SetInitActionInitializer();

			TRHIBufferInitializer<uint8> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
			Initializer.WriteArray(Expected_Offsets);
			FBufferRHIRef BufferRHI = Initializer.Finalize();

			const bool bDataMatches = VerifyBufferContents(*FString::Printf(TEXT("%s - Initializer"), *TestName), RHICmdList, BufferRHI, Expected_Offsets);
			bResult = bDataMatches ? bResult : false;
		}

		{
			// Create from array
			FResourceArrayUploadArrayView UploadView(Expected_Offsets);
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc(CommonCreateDesc)
				.SetInitActionResourceArray(&UploadView);

			FBufferRHIRef BufferRHI = RHICmdList.CreateBuffer(CreateDesc);

			const bool bDataMatches = VerifyBufferContents(*FString::Printf(TEXT("%s - ResourceArray"), *TestName), RHICmdList, BufferRHI, Expected_Offsets);
			bResult = bDataMatches ? bResult : false;
		}

		return bResult;
	}

	static bool Test_RHICreateBuffer(FRHICommandListImmediate& RHICmdList)
	{
		bool bResult = true;

		RUN_TEST(Test_RHICreateBuffer_Buffer(RHICmdList, 0));
		RUN_TEST(Test_RHICreateBuffer_Buffer(RHICmdList, 1));
		RUN_TEST(Test_RHICreateBuffer_Buffer(RHICmdList, 4));
		RUN_TEST(Test_RHICreateBuffer_Buffer(RHICmdList, 2048));

		return bResult;
	}

	static bool Test_RHICreateBuffer_Parallel(FRHICommandListImmediate& RHICmdList)
	{
		if (!GRHISupportsMultithreadedResources)
		{
			return true;
		}

		SCOPED_NAMED_EVENT_TEXT("Test_RHICreateBuffer_Parallel", FColor::Magenta);

		const int32 NumBuffersToCreate = 256;

		FRandomStream Rng(0x5bf8a575);

		TArray<TRefCountPtr<FRHIBuffer>> Buffers;
		Buffers.SetNum(NumBuffersToCreate);

		TArray<int32> RandomNumberPerBuffer;
		RandomNumberPerBuffer.SetNum(NumBuffersToCreate);

		for (int32 Index = 0; Index < NumBuffersToCreate; ++Index)
		{
			RandomNumberPerBuffer[Index] = Rng.RandRange(1, 65536);
		}

		ParallelDispatchCommands(RHICmdList, NumBuffersToCreate, [&](FRHICommandList& InRHICmdList, int32 Index, int32 Num)
		{
			SCOPED_NAMED_EVENT_TEXT("TestCreateBuffer", FColor::Magenta);

			const int32 Random = RandomNumberPerBuffer[Index];
			const int32 BufferSize = Align(Random, 16);
			const int32 BufferStride = 4;

			FRHIBufferCreateDesc Desc =
				FRHIBufferCreateDesc::Create(TEXT("Buffer"), BufferSize, 0, EBufferUsageFlags::VertexBuffer | EBufferUsageFlags::SourceCopy)
				.SetInitialState(ERHIAccess::CopySrc)
				.SetInitActionInitializer();

			switch (Random % 3)
			{
			case 0:
				Desc.AddUsage(EBufferUsageFlags::Static);
				break;
			case 1:
				Desc.AddUsage(EBufferUsageFlags::Dynamic);
				break;
			case 2:
				Desc.AddUsage(EBufferUsageFlags::Volatile);
				break;
			}

			TRHIBufferInitializer<uint32> Initializer = InRHICmdList.CreateBufferInitializer(Desc);

			uint32 NumDWORDs = BufferSize >> 2;

			for (uint32 DataIndex = 0; DataIndex < NumDWORDs; ++DataIndex)
			{
				Initializer[DataIndex] = DataIndex;
			}

			Buffers[Index] = Initializer.Finalize();
		});

		TArray<FRHIBuffer*> BufferPtrs;
		BufferPtrs.Reserve(Buffers.Num());

		for (int32 Index = 0; Index < NumBuffersToCreate; ++Index)
		{
			BufferPtrs.Emplace(Buffers[Index]);
		}

		return VerifyBufferContents(TEXT("Test_RHICreateBuffer_Parallel"), RHICmdList, BufferPtrs, [&](int32, void* Ptr, uint32 NumBytes)
		{
			uint32* Data = (uint32*)Ptr;
			uint32 NumDWORDs = NumBytes >> 2;

			for (uint32 Index = 0; Index < NumDWORDs; Index++)
			{
				if (Data[Index] != Index)
				{
					return false;
				}
			}

			return true;
		});
	}
};
