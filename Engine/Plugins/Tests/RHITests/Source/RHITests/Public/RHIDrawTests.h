// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHITestsCommon.h"

class FRHIDrawTests
{
public:

	static bool Test_DrawBaseVertexAndInstanceDirect(FRHICommandListImmediate& RHICmdList);
	static bool Test_DrawBaseVertexAndInstanceIndirect(FRHICommandListImmediate& RHICmdList);
	static bool Test_MultiDrawIndirect(FRHICommandListImmediate& RHICmdList);

private:

	enum class EDrawKind
	{
		Direct,
		Indirect,
	};

	static bool InternalDrawBaseVertexAndInstance(FRHICommandListImmediate& RHICmdList, EDrawKind DrawKind, const TCHAR* TestName);
};

