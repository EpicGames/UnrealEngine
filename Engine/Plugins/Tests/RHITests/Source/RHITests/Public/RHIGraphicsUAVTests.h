// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHITestsCommon.h"

/**
 * Set of tests for binding UAVs to non-compute shaders
 */

class FRHIGraphicsUAVTests
{
public:

	static bool Test_GraphicsUAV_PixelShader(FRHICommandListImmediate& RHICmdList);
	static bool Test_GraphicsUAV_VertexShader(FRHICommandListImmediate& RHICmdList);
};

