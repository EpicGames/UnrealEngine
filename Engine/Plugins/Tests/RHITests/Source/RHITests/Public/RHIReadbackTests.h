// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHITestsCommon.h"

class FRHIReadbackTests
{
public:
	static bool Test_BufferReadback(FRHICommandListImmediate& RHICmdList);
	static bool Test_TextureReadback(FRHICommandListImmediate& RHICmdList);
};
