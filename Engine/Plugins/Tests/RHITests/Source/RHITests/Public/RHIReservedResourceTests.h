// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHITestsCommon.h"

class FRHIReservedResourceTests
{
public:

	static bool Test_ReservedResource_CreateTexture(FRHICommandListImmediate& RHICmdList);
	static bool Test_ReservedResource_CreateTextureWithMips(FRHICommandListImmediate& RHICmdList);
	static bool Test_ReservedResource_CreateVolumeTexture(FRHICommandListImmediate& RHICmdList);
	static bool Test_ReservedResource_CreateBuffer(FRHICommandListImmediate& RHICmdList);
	static bool Test_ReservedResource_CommitBuffer(FRHICommandListImmediate& RHICmdList);
	static bool Test_ReservedResource_DecommitBuffer(FRHICommandListImmediate& RHICmdList);
};

