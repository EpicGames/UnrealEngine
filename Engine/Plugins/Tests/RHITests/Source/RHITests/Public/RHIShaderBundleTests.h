// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHITestsCommon.h"

class FRHIShaderBundleTests
{
public:
	static bool Test_ComputeShaderBundle(FRHICommandListImmediate& RHICmdList);
	static bool Test_GraphicsShaderBundle_MSPS(FRHICommandListImmediate& RHICmdList);
	static bool Test_GraphicsShaderBundle_VSPS(FRHICommandListImmediate& RHICmdList);
};
