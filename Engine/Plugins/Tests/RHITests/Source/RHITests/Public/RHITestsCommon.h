// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RenderUtils.h"
#include "RenderingThread.h"
#include "RHIResourceUtils.h"

DECLARE_LOG_CATEGORY_EXTERN(LogRHIUnitTestCommandlet, Log, All);

bool ShouldRunRHITest(const TCHAR* Name);
#define RUN_TEST(x) do { if (ShouldRunRHITest(TEXT(#x))) { bool Ret = x; bResult = bResult && Ret; } } while (false)

bool IsZeroMem(const void* Ptr, uint32 Size);

bool RunOnRenderThreadSynchronous(TFunctionRef<bool(FRHICommandListImmediate&)> TestFunc);

template <typename ValueType>
static inline FString ClearValueToString(const ValueType & ClearValue)
{
	if constexpr (std::is_floating_point_v<decltype(ClearValue.X)>)
	{
		return FString::Printf(TEXT("%f %f %f %f"), ClearValue.X, ClearValue.Y, ClearValue.Z, ClearValue.W);
	}
	else
	{
		return FString::Printf(TEXT("0x%08x 0x%08x 0x%08x 0x%08x"), ClearValue.X, ClearValue.Y, ClearValue.Z, ClearValue.W);
	}
}
