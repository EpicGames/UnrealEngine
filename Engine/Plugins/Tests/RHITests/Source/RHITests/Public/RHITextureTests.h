// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RHITestsCommon.h"
#include <type_traits>
#include "Math/PackedVector.h"
#include "RHIGPUReadback.h"

#include "RHISurfaceDataConversion.h"

//UE_DISABLE_OPTIMIZATION

class FRHITextureTests
{
	// This expects Texture to be in the CopySrc state.
	static bool VerifyTextureContents(const TCHAR* TestName, FRHICommandListImmediate& RHICmdList, FRHITexture* Texture, TFunctionRef<bool(void* Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 MipIndex, uint32 SliceIndex)> VerifyCallback);

public:
	template <typename ValueType, uint32 NumTestBytes>
	static bool RunTest_UAVClear_Texture(FRHICommandListImmediate& RHICmdList, const FString& TestName, FRHITexture* TextureRHI, uint32 MipIndex, const ValueType& ClearValue, void(FRHIComputeCommandList::* ClearPtr)(FRHIUnorderedAccessView*, ValueType const&), const uint8(&TestValue)[NumTestBytes])
	{
		bool bResult0, bResult1;
		{
			RHICmdList.Transition(FRHITransitionInfo(TextureRHI, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

			// Test clear whole resource to zero
			for (uint32 Mip = 0; Mip < TextureRHI->GetNumMips(); ++Mip)
			{
				FUnorderedAccessViewRHIRef MipUAV = RHICmdList.CreateUnorderedAccessView(
					TextureRHI, 
					FRHIViewDesc::CreateTextureUAV()
						.SetDimensionFromTexture(TextureRHI)
						.SetMipLevel(Mip));

				ValueType ZerosValue;
				FMemory::Memset(&ZerosValue, 0, sizeof(ZerosValue));
				(RHICmdList.*ClearPtr)(MipUAV, ZerosValue);
			}

			auto VerifyMip = [&](void* Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 CurrentMipIndex, uint32 CurrentSliceIndex, bool bShouldBeZero)
			{
				uint32 BytesPerPixel = GPixelFormats[TextureRHI->GetFormat()].BlockBytes;
				uint32 NumBytes = Width * Height * BytesPerPixel;
				check(NumBytes % NumTestBytes == 0);

				// This is the specific mip we're targeting.
				// Verify the mip entirely matches the clear value.

				for (uint32 Y = 0; Y < MipHeight; ++Y)
				{
					uint8* Row = ((uint8*)Ptr) + (Y * Width * BytesPerPixel);

					// Verify row within mip stride bounds matches the expected clear value
					for (uint32 X = 0; X < MipWidth; ++X)
					{
						uint8* Pixel = Row + X * BytesPerPixel;

						if (bShouldBeZero)
						{
							if (!IsZeroMem(Pixel, NumTestBytes))
								return false;
						}
						else
						{
							if (FMemory::Memcmp(Pixel, TestValue, NumTestBytes) != 0)
								return false;
						}
					}
				}

				return true;
			};

			auto VerifyMipIsZero = [&](void* Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 CurrentMipIndex, uint32 CurrentSliceIndex)
			{
				return VerifyMip(Ptr, MipWidth, MipHeight, Width, Height, CurrentMipIndex, CurrentSliceIndex, true);
			};

			RHICmdList.Transition(FRHITransitionInfo(TextureRHI, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));
			bResult0 = VerifyTextureContents(*FString::Printf(TEXT("%s - clear whole resource to zero"), *TestName), RHICmdList, TextureRHI, VerifyMipIsZero);

			// Clear the selected mip index to the provided value
			
			RHICmdList.Transition(FRHITransitionInfo(TextureRHI, ERHIAccess::CopySrc, ERHIAccess::UAVCompute));
			FUnorderedAccessViewRHIRef SpecificMipUAV = RHICmdList.CreateUnorderedAccessView(
				TextureRHI, 
				FRHIViewDesc::CreateTextureUAV()
					.SetDimensionFromTexture(TextureRHI)
					.SetMipLevel(MipIndex));
			(RHICmdList.*ClearPtr)(SpecificMipUAV, ClearValue);

			RHICmdList.Transition(FRHITransitionInfo(TextureRHI, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));
			bResult1 = VerifyTextureContents(*FString::Printf(TEXT("%s - clear mip %d to (%s)"), *TestName, MipIndex, *ClearValueToString(ClearValue)), RHICmdList, TextureRHI,
				[&](void* Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 CurrentMipIndex, uint32 CurrentSliceIndex)
			{
				return VerifyMip(Ptr, MipWidth, MipHeight, Width, Height, CurrentMipIndex, CurrentSliceIndex, CurrentMipIndex != MipIndex);
			});
		}

		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

		return bResult0 && bResult1;
	}

	template <typename ValueType, uint32 NumTestBytes>
	static bool Test_RHIClearUAV_Texture2D_WithParams(FRHICommandListImmediate& RHICmdList, uint32 NumMips, uint32 NumSlices, uint32 Width, uint32 Height, EPixelFormat Format, const ValueType& ClearValue, void(FRHIComputeCommandList::* ClearPtr)(FRHIUnorderedAccessView*, ValueType const&), const uint8(&TestValue)[NumTestBytes])
	{
		bool bResult = true;
		FString TestName = FString::Printf(TEXT("Test_RHIClearUAV_Texture2D (%dx%d, %d Slice(s), %d Mip(s)) - %s"), Width, Height, NumMips, NumSlices, *ClearValueToString(ClearValue));

		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc(*TestName, (NumSlices == 1) ? ETextureDimension::Texture2D : ETextureDimension::Texture2DArray)
				.SetFormat(Format)
				.SetExtent(Width, Height)
				.SetArraySize(NumSlices)
				.SetNumMips(NumMips)
				.SetFlags(ETextureCreateFlags::UAV | ETextureCreateFlags::ShaderResource);

			FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);

			for (uint32 Mip = 0; Mip < NumMips; ++Mip)
			{
				RUN_TEST(RunTest_UAVClear_Texture(RHICmdList, *TestName, Texture.GetReference(), Mip, ClearValue, ClearPtr, TestValue));
			}
		}
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

		return bResult;
	}

	template <typename ValueType, uint32 NumTestBytes>
	static bool Test_RHIClearUAV_Texture2D_Impl(FRHICommandListImmediate& RHICmdList, EPixelFormat Format, const ValueType& ClearValue, void(FRHIComputeCommandList::* ClearPtr)(FRHIUnorderedAccessView*, ValueType const&), const uint8(&TestValue)[NumTestBytes])
	{
		bool bResult = true;

		// Single Mip, Square
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 1, 1, 32, 32, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 1, 4, 32, 32, Format, ClearValue, ClearPtr, TestValue));

		// Multiple Mip, Square
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 4, 1, 32, 32, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 4, 4, 32, 32, Format, ClearValue, ClearPtr, TestValue));

		// Single Mip, pow2 Rectangle
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 1, 1, 16, 32, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 1, 1, 32, 16, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 1, 4, 16, 32, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 1, 4, 32, 16, Format, ClearValue, ClearPtr, TestValue));

		// Multiple Mip, pow2 Rectangle
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 4, 1, 16, 32, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 4, 1, 32, 16, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 4, 4, 16, 32, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 4, 4, 32, 16, Format, ClearValue, ClearPtr, TestValue));

		// Multiple Mip, Odd-sized
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 4, 1, 17, 23, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 4, 1, 23, 17, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 4, 4, 17, 23, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture2D_WithParams(RHICmdList, 4, 4, 23, 17, Format, ClearValue, ClearPtr, TestValue));

		return bResult;
	}

	static bool Test_RHIClearUAV_Texture2D(FRHICommandListImmediate& RHICmdList)
	{
		bool bResult = true;

		// Float       32-bit     16-bit
		// 0.2345  = 0x3e7020c5 | 0x3381
		// 0.8499  = 0x3f59930c | 0x3acc
		// 0.00145 = 0x3abe0ded | 0x15f0
		// 0.417   = 0x3ed58106 | 0x36ac
		const FVector4f ClearValueFloat(0.2345f, 0.8499f, 0.417f, 0.00145f);
		const FUintVector4 ClearValueUint32(0x01234567, 0x89abcdef, 0x8899aabb, 0xccddeeff);

		RUN_TEST(Test_RHIClearUAV_Texture2D_Impl(RHICmdList, PF_FloatRGBA, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0x81, 0x33, 0xcc, 0x3a, 0xac, 0x36, 0xf0, 0x15 }));
		RUN_TEST(Test_RHIClearUAV_Texture2D_Impl(RHICmdList, PF_R32_UINT, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01 }));

		return bResult;
	}

	template <typename ValueType, uint32 NumTestBytes>
	static bool Test_RHIClearUAV_Texture3D_WithParams(FRHICommandListImmediate& RHICmdList, uint32 NumMips, uint32 Width, uint32 Height, uint32 Depth, EPixelFormat Format, const ValueType& ClearValue, void(FRHIComputeCommandList::* ClearPtr)(FRHIUnorderedAccessView*, ValueType const&), const uint8(&TestValue)[NumTestBytes])
	{
		FString TestName = FString::Printf(TEXT("Test_RHIClearUAVUint_Texture3D (%dx%dx%d, %d Mip(s))"), Width, Height, Depth, NumMips);

		bool bResult = true;

		{
			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create3D(*TestName, Width, Height, Depth, Format)
				.SetNumMips(NumMips)
				.SetFlags(ETextureCreateFlags::UAV | ETextureCreateFlags::ShaderResource);

			FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);

			for (uint32 Mip = 0; Mip < NumMips; ++Mip)
			{
				RUN_TEST(RunTest_UAVClear_Texture(RHICmdList, *TestName, Texture.GetReference(), Mip, ClearValue, ClearPtr, TestValue));
			}
		}
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

		return bResult;
	}

	template <typename ValueType, uint32 NumTestBytes>
	static bool Test_RHIClearUAV_Texture3D_Impl(FRHICommandListImmediate& RHICmdList, EPixelFormat Format, const ValueType& ClearValue, void(FRHIComputeCommandList::* ClearPtr)(FRHIUnorderedAccessView*, ValueType const&), const uint8(&TestValue)[NumTestBytes])
	{
		bool bResult = true;

		// Single Mip, Cube
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 1, 32, 32, 32, Format, ClearValue, ClearPtr, TestValue));

		// Multiple Mip, Cube
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 4, 32, 32, 32, Format, ClearValue, ClearPtr, TestValue));

		// Single Mip, pow2 Cuboid
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 1, 16, 16, 32, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 1, 16, 32, 16, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 1, 32, 16, 16, Format, ClearValue, ClearPtr, TestValue));

		// Multiple Mip, pow2 Cuboid
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 4, 16, 16, 32, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 4, 16, 32, 16, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 4, 32, 16, 16, Format, ClearValue, ClearPtr, TestValue));

		// Multiple Mip, Odd-sized cuboid
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 4, 17, 23, 29, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 4, 29, 17, 23, Format, ClearValue, ClearPtr, TestValue));
		RUN_TEST(Test_RHIClearUAV_Texture3D_WithParams(RHICmdList, 4, 23, 29, 17, Format, ClearValue, ClearPtr, TestValue));

		return bResult;
	}

	static bool Test_RHIClearUAV_Texture3D(FRHICommandListImmediate& RHICmdList)
	{
		bool bResult = true;

		// Float       32-bit     16-bit
		// 0.2345  = 0x3e7020c5 | 0x3381
		// 0.8499  = 0x3f59930c | 0x3acc
		// 0.00145 = 0x3abe0ded | 0x15f0
		// 0.417   = 0x3ed58106 | 0x36ac
		const FVector4f ClearValueFloat(0.2345f, 0.8499f, 0.417f, 0.00145f);
		const FUintVector4 ClearValueUint32(0x01234567, 0x89abcdef, 0x8899aabb, 0xccddeeff);

		RUN_TEST(Test_RHIClearUAV_Texture3D_Impl(RHICmdList, PF_FloatRGBA, ClearValueFloat, &FRHICommandListImmediate::ClearUAVFloat, { 0x81, 0x33, 0xcc, 0x3a, 0xac, 0x36, 0xf0, 0x15 }));
		RUN_TEST(Test_RHIClearUAV_Texture3D_Impl(RHICmdList, PF_R32_UINT, ClearValueUint32, &FRHICommandListImmediate::ClearUAVUint, { 0x67, 0x45, 0x23, 0x01 }));

		return bResult;
	}

	static bool Test_RHIFormat_WithParams(FRHICommandListImmediate& RHICmdList, EPixelFormat ResourceFormat, EPixelFormat SRVFormat, EPixelFormat UAVFormat, ETextureCreateFlags Flags)
	{
		const uint32 Width = 32;
		const uint32 Height = 32;

		bool bResult = true;
		FString TestName = FString::Printf(TEXT("Test_RHIFormat (%s, %s, %s, %lld)"), GPixelFormats[ResourceFormat].Name, GPixelFormats[SRVFormat].Name, GPixelFormats[UAVFormat].Name, EnumToUnderlyingType(Flags));
		{
			FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(*TestName, Width, Height, ResourceFormat)
				.SetFlags(Flags);
			
			if(ResourceFormat != SRVFormat && SRVFormat != PF_Unknown)
			{
				Desc.AddAliasbleFormat(SRVFormat);
			}

			FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);
			bResult = (Texture != nullptr);

			if (Texture && SRVFormat != PF_Unknown)
			{
				FShaderResourceViewRHIRef SRV = RHICmdList.CreateShaderResourceView(
					Texture, 
					FRHIViewDesc::CreateTextureSRV()
						.SetDimensionFromTexture(Texture)
						.SetFormat(SRVFormat)
						.SetMipRange(0, 1));
				bResult = (SRV != nullptr);
			}

			// TODO
			if (Texture && UAVFormat != PF_Unknown)
			{
				//RHICmdList.CreateUnorderedAccessView(Texture, 0, UAVFormat);
			}
		}
		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

		if (bResult)
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"%ls\"", *TestName);
		}

		return bResult;
	}

	static bool Test_RHIFormat_RenderTargetFormat(FRHICommandListImmediate& RHICmdList, EPixelFormat Format, bool bAllowUAV)
	{
		bool bResult = true;
		if (EnumHasAnyFlags(GPixelFormats[Format].Capabilities, EPixelFormatCapabilities::RenderTarget))
		{
			RUN_TEST(Test_RHIFormat_WithParams(RHICmdList, Format, PF_Unknown, PF_Unknown, TexCreate_RenderTargetable));
			RUN_TEST(Test_RHIFormat_WithParams(RHICmdList, Format, Format, PF_Unknown, TexCreate_RenderTargetable | TexCreate_ShaderResource));
			if (bAllowUAV)
			{
				RUN_TEST(Test_RHIFormat_WithParams(RHICmdList, Format, PF_Unknown, Format, TexCreate_RenderTargetable | TexCreate_UAV));
				RUN_TEST(Test_RHIFormat_WithParams(RHICmdList, Format, Format, Format, TexCreate_RenderTargetable | TexCreate_ShaderResource | TexCreate_UAV));
			}
		}
		else
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Skipping test for lack of format support. \"Test_RHIFormat_RenderTargetFormat (%ls)\"", GPixelFormats[Format].Name);
		}
		return bResult;
	}

	static bool Test_RHIFormat_DepthFormat(FRHICommandListImmediate& RHICmdList, EPixelFormat ResourceFormat)
	{
		bool bResult = true;
		if (EnumHasAnyFlags(GPixelFormats[ResourceFormat].Capabilities, EPixelFormatCapabilities::DepthStencil))
		{
			RUN_TEST(Test_RHIFormat_WithParams(RHICmdList, ResourceFormat, PF_Unknown, PF_Unknown, TexCreate_DepthStencilTargetable));
			RUN_TEST(Test_RHIFormat_WithParams(RHICmdList, ResourceFormat, ResourceFormat, PF_Unknown, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource));

			if (ResourceFormat == PF_DepthStencil)
			{
				RUN_TEST(Test_RHIFormat_WithParams(RHICmdList, ResourceFormat, PF_X24_G8, PF_Unknown, TexCreate_DepthStencilTargetable | TexCreate_ShaderResource));
			}
		}
		else
		{
			UE_LOGF(LogRHIUnitTestCommandlet, Display, "Skipping test for lack of format support. \"Test_RHIFormat_DepthFormat (%ls)\"", GPixelFormats[ResourceFormat].Name);
		}
		return bResult;
	}

	static bool Test_RHIFormats(FRHICommandListImmediate& RHICmdList)
	{
		bool bResult = true;
		RUN_TEST(Test_RHIFormat_RenderTargetFormat(RHICmdList, PF_R32_FLOAT, true));

		RUN_TEST(Test_RHIFormat_DepthFormat(RHICmdList, PF_DepthStencil));
		RUN_TEST(Test_RHIFormat_DepthFormat(RHICmdList, PF_ShadowDepth));
		RUN_TEST(Test_RHIFormat_DepthFormat(RHICmdList, PF_R32_FLOAT));
		RUN_TEST(Test_RHIFormat_DepthFormat(RHICmdList, PF_D24));

		return bResult;
	}

	template <typename T, std::enable_if_t<std::is_same<T,float>::value || std::is_same<T,FFloat16>::value, bool> = true>
	static void FillValues(uint32 NumComponents, uint32 Width, uint32 Height, T* Values)
	{
		uint32 index = 0;
		for (uint32 Y = 0; Y < Height; Y++)
		{
			for (uint32 X = 0; X < Width; X++)
			{
				for (uint32 Component = 0; Component < NumComponents; Component++)
				{
					Values[index] = float(index) / (float)(NumComponents * Width * Height);
					index++;
				}
			}
		}
	}

	template <typename T, std::enable_if_t<std::is_unsigned<T>::value, bool> = true>
	static void FillValues(uint32 NumComponents, uint32 Width, uint32 Height, T* Values)
	{
		static_assert(std::is_integral<T>::value, "Integral required.");
		uint64 index = 0;
		uint64 Value = 0;
		const T MinInt = T(1) << (sizeof(T) * 8 - 1);
		for (uint32 Y = 0; Y < Height; Y++)
		{
			for (uint32 X = 0; X < Width; X++)
			{
				for (uint32 Component = 0; Component < NumComponents; Component++)
				{
					// SNORM has two values for -1.0 (for example 0x80 and 0x81) so that 0.0 exists in the encoding
					// Skip the lowest signed value as texture copy/update might write different encoding
					if (T(Value) == MinInt)
					{
						Value++;
					}
					Values[index++] = T(Value++);
				}
			}
		}
	}

	template <typename T, std::enable_if_t<std::is_same<T,FFloat3Packed>::value, bool> = true>
	static void FillValues(uint32 NumComponents, uint32 Width, uint32 Height, T* Values)
	{
		uint32 ValueIndex = 0;
		uint32 ComponentIndex = 0;
		for (uint32 Y = 0; Y < Height; Y++)
		{
			for (uint32 X = 0; X < Width; X++)
			{
				FLinearColor Color;
				Color.R = float(ComponentIndex++ % (1 << 11)) / (float)(1 << 11);
				Color.G = float(ComponentIndex++ % (1 << 11)) / (float)(1 << 11);
				Color.B = float(ComponentIndex++ % (1 << 11)) / (float)(1 << 11);
				Color.A = 0.0f;
				Values[ValueIndex] = T(Color);
				ValueIndex++;
			}
		}
	}

	template <typename T, uint32 NumComponents, bool bInFloatUAV>
	struct FDataSource
	{
		static const bool bFloatUAV = bInFloatUAV;
		static const uint32 ElementSize = sizeof(T) * NumComponents;

		static void FillSourceData(uint32 Width, uint32 Height, uint8* OutSourceData)
		{
			FillValues(NumComponents, Width, Height, (T*)OutSourceData);
		}
	};

	typedef FDataSource< uint8,			1, true >	FDataSource8x1;
	typedef FDataSource< uint8,			2, true >	FDataSource8x2;
	typedef FDataSource< uint8,			3, true >	FDataSource8x3;
	typedef FDataSource< uint8,			4, true >	FDataSource8x4;
	typedef FDataSource< uint16,		1, true >	FDataSource16x1;
	typedef FDataSource< uint16,		2, true >	FDataSource16x2;
	typedef FDataSource< uint16,		4, true >	FDataSource16x4;
	typedef FDataSource< uint32,		1, true >	FDataSource32x1;
	typedef FDataSource< uint32,		2, true >	FDataSource32x2;
	typedef FDataSource< uint32,		4, true >	FDataSource32x4;
	typedef FDataSource< FFloat16,		1, true >	FDataSource16x1F;
	typedef FDataSource< FFloat16,		2, true >	FDataSource16x2F;
	typedef FDataSource< FFloat16,		4, true >	FDataSource16x4F;
	typedef FDataSource< float,			1, true >	FDataSource32x1F;
	typedef FDataSource< float,			2, true >	FDataSource32x2F;
	typedef FDataSource< float,			3, true >	FDataSource32x3F;
	typedef FDataSource< float,			4, true >	FDataSource32x4F;
	typedef FDataSource< uint8,			1, false >	FDataSource8x1UInt;
	typedef FDataSource< uint8,			2, false >	FDataSource8x2UInt;
	typedef FDataSource< uint8,			4, false >	FDataSource8x4UInt;
	typedef FDataSource< uint16,		1, false >	FDataSource16x1UInt;
	typedef FDataSource< uint16,		2, false >	FDataSource16x2UInt;
	typedef FDataSource< uint16,		4, false >	FDataSource16x4UInt;
	typedef FDataSource< uint32,		1, false >	FDataSource32x1UInt;
	typedef FDataSource< uint32,		2, false >	FDataSource32x2UInt;
	typedef FDataSource< uint32,		3, false >	FDataSource32x3UInt;
	typedef FDataSource< uint32,		4, false >	FDataSource32x4UInt;
	typedef FDataSource< uint64,		1, false >	FDataSource64x1UInt;
	typedef FDataSource< FFloat3Packed,	1, true >	FDataSource11_11_10F;

	enum class ETextureUpdateType
	{
		Texture2D,
		TextureFromBuffer,
		Texture3D
	};

	static const TCHAR* GetName(ETextureUpdateType UpdateType)
	{
		switch (UpdateType)
		{
		case ETextureUpdateType::Texture2D:
			return TEXT("Texture2D");
		case ETextureUpdateType::TextureFromBuffer:
			return TEXT("TextureFromBuffer");
		case ETextureUpdateType::Texture3D:
			return TEXT("Texture3D");
		}
		return TEXT("");
	}

	template <typename SourceType>
	static bool Test_UpdateTexture_Impl(FRHICommandListImmediate& RHICmdList, FString TestName, FRHITexture* Texture, const FUpdateTextureRegion2D& Region, uint32 SourceSize, uint32 SourcePitch, const uint8* SourceData, const uint8* ZeroData, ETextureUpdateType UpdateType)
	{
		bool bResult = true;
		
		auto VerifyMip = [&](void* Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 CurrentMipIndex, uint32 CurrentSliceIndex, bool bAlwaysZero)
		{
			uint32 BytesPerPixel = GPixelFormats[Texture->GetFormat()].BlockBytes;
			check(CurrentMipIndex == 0);
			check(BytesPerPixel == SourceType::ElementSize);

			for (uint32 Y = 0; Y < MipHeight; ++Y)
			{
				uint8* Row = ((uint8*)Ptr) + (Y * Width * BytesPerPixel);

				// Verify row within mip stride bounds matches the expected value
				for (uint32 X = 0; X < MipWidth; ++X)
				{
					uint8* Pixel = Row + X * BytesPerPixel;

					bool bShouldBeZero = bAlwaysZero || X < Region.DestX || Y < Region.DestY || X >= (Region.DestX + Region.Width) || Y >= (Region.DestY + Region.Height);
					if (bShouldBeZero)
					{
						if (!IsZeroMem(Pixel, BytesPerPixel))
							return false;
					}
					else
					{
						const uint8* TestPixel = SourceData + (Region.SrcX + X - Region.DestX) * BytesPerPixel + (Region.SrcY + Y - Region.DestY) * SourcePitch;
						if (FMemory::Memcmp(Pixel, TestPixel, BytesPerPixel) != 0)
							return false;
					}
				}
			}

			return true;
		};

		RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::Unknown, ERHIAccess::UAVCompute));

		// clear to zero
		if (UpdateType == ETextureUpdateType::Texture3D)
		{
			for (uint32 DepthSlice = 0; DepthSlice < Texture->GetSizeZ(); ++DepthSlice)
			{
				FUpdateTextureRegion3D ZeroRegion = { 0, 0, DepthSlice, 0, 0, 0, Texture->GetSizeX(), Texture->GetSizeY(), 1 };
				RHICmdList.UpdateTexture3D(Texture, 0, ZeroRegion, SourcePitch, SourcePitch * ZeroRegion.Height, ZeroData);
			}
		}
		else
		{
			FUpdateTextureRegion2D ZeroRegion = { 0, 0, 0, 0, Texture->GetSizeX(), Texture->GetSizeY() };
			RHICmdList.UpdateTexture2D(Texture, 0, ZeroRegion, SourcePitch, ZeroData);
		}

		RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::UAVCompute, ERHIAccess::CopySrc));

		bResult &= VerifyTextureContents(*FString::Printf(TEXT("%s - clear whole resource to zero"), *TestName), RHICmdList, Texture,
			[&](void* Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 CurrentMipIndex, uint32 CurrentSliceIndex)
		{
			return VerifyMip(Ptr, MipWidth, MipHeight, Width, Height, CurrentMipIndex, CurrentSliceIndex, true);
		});

		if (UpdateType == ETextureUpdateType::Texture2D)
		{
			RHICmdList.UpdateTexture2D(Texture, 0, Region, SourcePitch, SourceData);
		}
		else if (UpdateType == ETextureUpdateType::TextureFromBuffer)
		{
			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::Create(TEXT("SourceUpdateBuffer"), SourceSize, SourceType::ElementSize, EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Static | EBufferUsageFlags::KeepCPUAccessible)
				.SetInitialState(ERHIAccess::SRVCompute)
				.SetInitActionInitializer();

			FRHIBufferInitializer Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
			Initializer.WriteData(SourceData, SourceSize);
			FBufferRHIRef Buffer = Initializer.Finalize();

			RHICmdList.UpdateFromBufferTexture2D(Texture, 0, Region, SourcePitch, Buffer, 0);
		}
		else
		{
			check(UpdateType == ETextureUpdateType::Texture3D);

			for (uint32 DepthSlice = 0; DepthSlice < Texture->GetSizeZ(); ++DepthSlice)
			{
				FUpdateTextureRegion3D Region3D = { Region.DestX, Region.DestY, DepthSlice, Region.SrcX, Region.SrcY, 0, Region.Width, Region.Height, 1 };
				RHICmdList.UpdateTexture3D(Texture, 0, Region3D, SourcePitch, SourcePitch * Texture->GetSizeY(), SourceData);
			}
		}

		// FIXME: we should define the state expected by the UpdateTexture calls, and how to properly synchronize subsequent access to the texture. Until then,
		// we'll insert a transition from Unknown here and hope for the best.
		RHICmdList.Transition(FRHITransitionInfo(Texture, ERHIAccess::Unknown, ERHIAccess::CopySrc));

		bResult &= VerifyTextureContents(*FString::Printf(TEXT("%s - update (%d,%d -> %d,%d)"), *TestName, Region.SrcX, Region.SrcY, Region.DestX, Region.DestY), RHICmdList, Texture,
			[&](void* Ptr, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 CurrentMipIndex, uint32 CurrentSliceIndex)
		{
			return VerifyMip(Ptr, MipWidth, MipHeight, Width, Height, CurrentMipIndex, CurrentSliceIndex, false);
		});

		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

		return bResult;
	}

	template <typename SourceType>
	static bool Test_UpdateTexture_Impl(FRHICommandListImmediate& RHICmdList, EPixelFormat Format, ETextureUpdateType UpdateType)
	{
		const uint32 TextureWidth = 128;
		const uint32 TextureHeight = 128;
		const uint32 SrcDataWidth = 128;
		const uint32 SrcDataHeight = 128;
		const uint32 UpdateWidth = 64;
		const uint32 UpdateHeight = 64;
		const uint32 UpdateDataPitch = SrcDataWidth * SourceType::ElementSize;
		const uint32 SrcDataSize = SrcDataWidth * SrcDataHeight * SourceType::ElementSize;
		uint8* UpdateData;
		uint8* ZeroData;

		FConcurrentLinearBulkObjectAllocator Allocator;
		UpdateData = Allocator.MallocArray<uint8>(SrcDataSize);
		ZeroData = Allocator.MallocArray<uint8>(SrcDataSize);
		SourceType::FillSourceData(SrcDataWidth, SrcDataHeight, UpdateData);
		FMemory::Memset(ZeroData, 0, SrcDataSize);

		bool bResult = true;
		FString TestName = FString::Printf(TEXT("Test_UpdateTexture (%s) (%s)"), GPixelFormats[Format].Name, GetName(UpdateType));
		{
			if (!GPixelFormats[Format].Supported)
			{
				UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test skipped (format not supported). \"%ls\"", *TestName);
				return true;
			}
			FRHITextureCreateDesc Desc;
			
			if (UpdateType == ETextureUpdateType::Texture3D)
			{
				const int32 TextureDepth = 4;

				if (!UE::PixelFormat::HasCapabilities(Format, EPixelFormatCapabilities::Texture3D))
				{
					UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test skipped (format not supported as Texture3D). \"%ls\"", *TestName);
					return true;
				}

				Desc = FRHITextureCreateDesc::Create3D(*TestName, FIntVector(TextureWidth, TextureHeight, TextureDepth), Format);
			}
			else
			{
				if (!UE::PixelFormat::HasCapabilities(Format, EPixelFormatCapabilities::Texture2D))
				{
					UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test skipped (format not supported as Texture2D). \"%ls\"", *TestName);
					return true;
				}

				Desc = FRHITextureCreateDesc::Create2D(*TestName, TextureWidth, TextureHeight, Format);
			}

			Desc.SetFlags(ETextureCreateFlags::ShaderResource);

			FTextureRHIRef Texture = RHICmdList.CreateTexture(Desc);
			if (Texture == nullptr)
			{
				UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test failed (couldn't create texture). \"%ls\"", *TestName);
				return false;
			}

			auto AxisSlotToCoord = [](uint32 Slot, uint32 TextureSize, uint32 UpdateSize) -> uint32
			{
				switch (Slot)
				{
				default: checkNoEntry();
				case 0: return 0;
				case 1: return (TextureSize / 2) - (UpdateSize / 2);
				case 2: return TextureSize - UpdateSize;
				}
			};

			FUpdateTextureRegion2D Region(0, 0, 0, 0, UpdateWidth, UpdateHeight);
			// Test destination offsets
			for (uint32 DestRow = 0; DestRow < 3; DestRow++)
			{
				Region.DestY = AxisSlotToCoord(DestRow, TextureHeight, UpdateHeight);
				for (uint32 DestColumn = 0; DestColumn < 3; DestColumn++)
				{
					Region.DestX = AxisSlotToCoord(DestColumn, TextureWidth, UpdateWidth);
					bResult &= Test_UpdateTexture_Impl<SourceType>(RHICmdList, TestName, Texture, Region, SrcDataSize, UpdateDataPitch, UpdateData, ZeroData, UpdateType);
				}
			}

			// @todo enable for all types when they support source offsets
			if (UpdateType == ETextureUpdateType::Texture2D)
			{
				Region = FUpdateTextureRegion2D(0, 0, 0, 0, UpdateWidth, UpdateHeight);

				// Test source offsets
				for (uint32 SrcRow = 0; SrcRow < 3; SrcRow++)
				{
					Region.SrcY = AxisSlotToCoord(SrcRow, SrcDataHeight, UpdateHeight);
					for (uint32 SrcColumn = 0; SrcColumn < 3; SrcColumn++)
					{
						Region.SrcX = AxisSlotToCoord(SrcColumn, SrcDataWidth, UpdateWidth);
						bResult &= Test_UpdateTexture_Impl<SourceType>(RHICmdList, TestName, Texture, Region, SrcDataSize, UpdateDataPitch, UpdateData, ZeroData, UpdateType);
					}
				}
			}
		}

		return bResult;
	}

	static bool Test_UpdateTexture_Impl(FRHICommandListImmediate& RHICmdList, ETextureUpdateType UpdateType)
	{
		bool bResult = true;
		auto RunTestOnFormat = [&](EPixelFormat Format)
		{
			switch (Format)
			{
			default:
				checkNoEntry();
			case PF_Unknown:
			case PF_DXT1:
			case PF_DXT3:
			case PF_DXT5:
			case PF_UYVY:
			case PF_DepthStencil:
			case PF_ShadowDepth:
			case PF_D24:
			case PF_BC5:
			case PF_A1:
			case PF_PVRTC2:
			case PF_PVRTC4:
			case PF_BC4:
			case PF_ATC_RGB:
			case PF_ATC_RGBA_E:
			case PF_ATC_RGBA_I:
			case PF_X24_G8:
			case PF_ETC1:
			case PF_ETC2_RGB:
			case PF_ETC2_RGBA:
			case PF_ASTC_4x4:
			case PF_ASTC_6x6:
			case PF_ASTC_8x8:
			case PF_ASTC_10x10:
			case PF_ASTC_12x12:
			case PF_BC6H:
			case PF_BC7:
			case PF_XGXR8:
			case PF_PLATFORM_HDR_0:
			case PF_PLATFORM_HDR_1:
			case PF_PLATFORM_HDR_2:
			case PF_NV12:
			case PF_ETC2_R11_EAC:
			case PF_ETC2_RG11_EAC:
			case PF_ASTC_4x4_HDR:
			case PF_ASTC_6x6_HDR:
			case PF_ASTC_8x8_HDR:
			case PF_ASTC_10x10_HDR:
			case PF_ASTC_12x12_HDR:
			case PF_R32G32B32F:
			case PF_R32G32B32_UINT:
			case PF_R32G32B32_SINT:
			case PF_R9G9B9EXP5:
			case PF_ASTC_4x4_NORM_RG:
			case PF_ASTC_6x6_NORM_RG:
			case PF_ASTC_8x8_NORM_RG:
			case PF_ASTC_10x10_NORM_RG:
			case PF_ASTC_12x12_NORM_RG:
			case PF_P010:
				break;

			case PF_G8:
			case PF_A8:
			case PF_L8:
			case PF_R8:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource8x1>(RHICmdList, Format, UpdateType));
				break;
			case PF_R8_UINT:
			case PF_R8_SINT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource8x1UInt>(RHICmdList, Format, UpdateType));
				break;
			case PF_V8U8:
			case PF_R8G8:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource8x2>(RHICmdList, Format, UpdateType));
				break;
			case PF_R8G8_UINT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource8x2UInt>(RHICmdList, Format, UpdateType));
				break;
			case PF_R8G8B8:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource8x3>(RHICmdList, Format, UpdateType));
				break;
			case PF_B8G8R8A8:
			case PF_R8G8B8A8:
			case PF_A8R8G8B8:
			case PF_R8G8B8A8_SNORM:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource8x4>(RHICmdList, Format, UpdateType));
				break;
			case PF_R8G8B8A8_UINT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource8x4UInt>(RHICmdList, Format, UpdateType));
				break;
			case PF_G16:
			case PF_R5G6B5_UNORM:	// only for the 16-bit pixel data
			case PF_B5G5R5A1_UNORM:	// only for the 16-bit pixel data
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource16x1>(RHICmdList, Format, UpdateType));
				break;
			case PF_R16_UINT:
			case PF_R16_SINT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource16x1UInt>(RHICmdList, Format, UpdateType));
				break;
			case PF_R16F:
			case PF_R16F_FILTER:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource16x1F>(RHICmdList, Format, UpdateType));
				break;
			case PF_G16R16:
			case PF_G16R16_SNORM:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource16x2>(RHICmdList, Format, UpdateType));
				break;
			case PF_R16G16_UINT:
			case PF_R16G16_SINT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource16x2UInt>(RHICmdList, Format, UpdateType));
				break;
			case PF_G16R16F:
			case PF_G16R16F_FILTER:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource16x2F>(RHICmdList, Format, UpdateType));
				break;
			case PF_A16B16G16R16:
			case PF_R16G16B16A16_UNORM:
			case PF_R16G16B16A16_SNORM:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource16x4>(RHICmdList, Format, UpdateType));
				break;
			case PF_R16G16B16A16_SINT:
			case PF_R16G16B16A16_UINT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource16x4UInt>(RHICmdList, Format, UpdateType));
				break;
			case PF_R32_SINT:
			case PF_R32_UINT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource32x1UInt>(RHICmdList, Format, UpdateType));
				break;
			case PF_R32_FLOAT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource32x1F>(RHICmdList, Format, UpdateType));
				break;
			case PF_G32R32F:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource32x2F>(RHICmdList, Format, UpdateType));
				break;
			case PF_R32G32_UINT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource32x2UInt>(RHICmdList, Format, UpdateType));
				break;
			case PF_A32B32G32R32F:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource32x4F>(RHICmdList, Format, UpdateType));
				break;
			case PF_R32G32B32A32_UINT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource32x4UInt>(RHICmdList, Format, UpdateType));
				break;
			case PF_R64_UINT:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource64x1UInt>(RHICmdList, Format, UpdateType));
				break;
			case PF_FloatRGB:
			case PF_FloatR11G11B10:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource11_11_10F>(RHICmdList, Format, UpdateType));
				break;
			case PF_A2B10G10R10:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource8x4>(RHICmdList, Format, UpdateType));
				break;
			case PF_FloatRGBA:
				RUN_TEST(Test_UpdateTexture_Impl<FDataSource16x4F>(RHICmdList, Format, UpdateType));
				break;
			}
		};
#define RUN_TEST_ON_FORMAT(Format) RunTestOnFormat(Format);
		FOREACH_ENUM_EPIXELFORMAT_CORE(RUN_TEST_ON_FORMAT);
#undef RUN_TEST_ON_FORMAT

		return bResult;
	}

	static bool Test_UpdateTexture(FRHICommandListImmediate& RHICmdList)
	{
		bool bResult = true;

		bResult &= Test_UpdateTexture_Impl(RHICmdList, ETextureUpdateType::Texture2D);

		if (GRHISupportsUpdateFromBufferTexture)
		{
			bResult &= Test_UpdateTexture_Impl(RHICmdList, ETextureUpdateType::TextureFromBuffer);
		}

		bResult &= Test_UpdateTexture_Impl(RHICmdList, ETextureUpdateType::Texture3D);

		return bResult;
	}

	//////////////////////////////////////////////////////////////////////////

	static bool Test_RHICopyTexture(FRHICommandListImmediate& RHICmdList)
	{
		bool bResult = true;

		const auto WriteTestData = [](void* Ptr, int32 Stride, FIntPoint Size, FIntPoint Offset, int32 ImageIndex)
		{
			int32 Index = ImageIndex + Offset.Y * Size.X + Offset.X;
			uint8* Bytes = (uint8*)Ptr;

			for (int32 Y = Offset.Y; Y < Offset.Y + Size.Y; Y++)
			{
				uint32* Row = (uint32*)(Bytes + Y * Stride);

				for (int32 X = Offset.X; X < Offset.X + Size.X; X++)
				{
					Row[X] = Index;
					Index++;
				}
			}
		};

		// Temp: FIntPoint::ZeroValue replaced with UE::Math::TIntPoint<int32>::ZeroValue to work around msvc lambda compiler bug
		const auto DumpTestData = [](const void* Ptr, int32 Stride, FIntPoint Size, FIntPoint Offset = UE::Math::TIntPoint<int32>::ZeroValue)
		{
			FString DataString;
			uint8* Bytes = (uint8*)Ptr;

			for (int32 Y = Offset.Y; Y < Offset.Y + Size.Y; Y++)
			{
				const uint32* Row = (const uint32*)(Bytes + Y * Stride);

				for (int32 X = Offset.X; X < Offset.X + Size.X; X++)
				{
					DataString += FString::Printf(TEXT(" %4d"), Row[X]);
				}

				DataString += TEXT("\n");
			}

			return DataString;
		};

		const auto CheckTestData = [&](const void* Ptr, int32 Stride, FIntPoint CheckSize, FIntPoint DataOffset, FIntPoint SrcSize, FIntPoint SrcOffset, int32 ImageIndex) -> bool
		{
			const uint8* Bytes = (uint8*)Ptr;
			int32 Index = ImageIndex + SrcOffset.Y * SrcSize.X + SrcOffset.X;

			for (int32 Y = DataOffset.Y; Y < DataOffset.Y + CheckSize.Y; Y++)
			{
				const uint32* Row = (const uint32*)(Bytes + Y * Stride);

				for (int32 X = DataOffset.X; X < DataOffset.X + CheckSize.X; X++, Index++)
				{
					if (Row[X] != Index)
					{
						UE_LOGF(LogRHIUnitTestCommandlet, Error,
							"Dumping Test Data:\n" "\tCoordinates: [%d, %d]\n" "\tExpected Value: %d\n" "\tActual Value: %d\n" "%ls",
							X, Y, Index, Row[X], *DumpTestData(Ptr, Stride, CheckSize, DataOffset));

						return false;
					}
				}

				Index += SrcSize.X - CheckSize.X;
			}

			return true;
		};

		{
			// Copy to identical texture
			{
				const TCHAR* TestName = TEXT("Copy Texture Whole");
				const FIntPoint Extent(16, 16);
				const int32 ImageIndex = 1;

				const FRHITextureCreateDesc SourceDesc =
					FRHITextureCreateDesc::Create2D(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopySrc);

				FTextureRHIRef SourceTexture = RHICmdList.CreateTexture(SourceDesc);

				uint32 Stride = 0;
				void* Data = RHICmdList.LockTexture2D(SourceTexture, 0, RLM_WriteOnly, Stride, false);
				WriteTestData(Data, Stride, Extent, FIntPoint::ZeroValue, ImageIndex);
				RHICmdList.UnlockTexture2D(SourceTexture, 0, false);

				const FRHITextureCreateDesc DestDesc =
					FRHITextureCreateDesc::Create2D(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopyDest);

				FTextureRHIRef DestTexture = RHICmdList.CreateTexture(DestDesc);
				RHICmdList.CopyTexture(SourceTexture, DestTexture, {});
				RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::CopyDest, ERHIAccess::CopySrc));

				RUN_TEST(VerifyTextureContents(TestName, RHICmdList, DestTexture, [&](void* Data, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 MipIndex, uint32 SliceIndex)
				{
					return CheckTestData(Data, Width * sizeof(uint32), Extent, FIntPoint::ZeroValue, Extent, FIntPoint::ZeroValue, ImageIndex);
				}));
			}

			// Copy textures with UNorm or sRGB settings (Dest should remain bitwise identical to Source)
			const auto CopyTextureSrgbLambda = [&](const TCHAR* TestName, bool bSourceSrgb, bool bDestSrgb)
			{
				const FIntPoint Extent(16, 16);
				const int32 ImageIndex = 1;

				const FRHITextureCreateDesc SourceDesc =
					FRHITextureCreateDesc::Create2D(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetFormat(PF_B8G8R8A8)
					.SetFlags(bSourceSrgb ? ETextureCreateFlags::SRGB : ETextureCreateFlags::None)
					.SetInitialState(ERHIAccess::CopySrc);

				FTextureRHIRef SourceTexture = RHICmdList.CreateTexture(SourceDesc);

				uint32 Stride = 0;
				void* Data = RHICmdList.LockTexture2D(SourceTexture, 0, RLM_WriteOnly, Stride, false);
				WriteTestData(Data, Stride, Extent, FIntPoint::ZeroValue, ImageIndex);
				RHICmdList.UnlockTexture2D(SourceTexture, 0, false);

				const FRHITextureCreateDesc DestDesc =
					FRHITextureCreateDesc::Create2D(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetFormat(PF_B8G8R8A8)
					.SetFlags(bDestSrgb ? ETextureCreateFlags::SRGB : ETextureCreateFlags::None)
					.SetInitialState(ERHIAccess::CopyDest);

				FTextureRHIRef DestTexture = RHICmdList.CreateTexture(DestDesc);
				RHICmdList.CopyTexture(SourceTexture, DestTexture, {});
				RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::CopyDest, ERHIAccess::CopySrc));

				RUN_TEST(VerifyTextureContents(TestName, RHICmdList, DestTexture, [&](void* Data, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 MipIndex, uint32 SliceIndex)
				{
					return CheckTestData(Data, Width * sizeof(uint32), Extent, FIntPoint::ZeroValue, Extent, FIntPoint::ZeroValue, ImageIndex);
				}));
			};

			CopyTextureSrgbLambda(TEXT("Copy Texture UNorm to UNorm"), false, false);
			CopyTextureSrgbLambda(TEXT("Copy Texture UNorm to sRGB"), false, true);
			CopyTextureSrgbLambda(TEXT("Copy Texture sRGB to UNorm"), true, false);
			CopyTextureSrgbLambda(TEXT("Copy Texture sRGB to sRGB"), true, true);

			// Copy mip to another texture.
			const auto CopyTextureMipsLambda = [&](bool bUseExplicitSize)
			{
				const FIntPoint Extent(32, 32);
				const int32 NumMips = 3;

				const FRHITextureCreateDesc SourceDesc =
					FRHITextureCreateDesc::Create2D(TEXT("Copy Texture Mips"))
					.SetExtent(Extent.X, Extent.Y)
					.SetNumMips(NumMips)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopySrc);

				FTextureRHIRef SourceTexture = RHICmdList.CreateTexture(SourceDesc);

				for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
				{
					uint32 Stride = 0;
					void* Data = RHICmdList.LockTexture2D(SourceTexture, MipIndex, RLM_WriteOnly, Stride, false);
					WriteTestData(Data, Stride, FIntPoint(Extent.X >> MipIndex, Extent.Y >> MipIndex), FIntPoint::ZeroValue, MipIndex);
					RHICmdList.UnlockTexture2D(SourceTexture, MipIndex, false);
				}

				for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
				{
					const FString TestName = FString::Printf(TEXT("Copy Texture Mips [Mip %d, Size Explicit: %s]"), MipIndex, bUseExplicitSize ? TEXT("Yes") : TEXT("No"));

					const FRHITextureCreateDesc DestDesc =
						FRHITextureCreateDesc::Create2D(*TestName)
						.SetExtent(Extent.X >> MipIndex, Extent.Y >> MipIndex)
						.SetFormat(PF_R32_UINT)
						.SetInitialState(ERHIAccess::CopySrc);

					FTextureRHIRef DestTexture = RHICmdList.CreateTexture(DestDesc);

					FRHICopyTextureInfo CopyInfo;
					CopyInfo.Size = bUseExplicitSize ? FIntVector(DestDesc.Extent.X, DestDesc.Extent.Y, 1) : FIntVector::ZeroValue;
					CopyInfo.SourceMipIndex = MipIndex;

					RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::CopySrc, ERHIAccess::CopyDest));
					RHICmdList.CopyTexture(SourceTexture, DestTexture, CopyInfo);
					RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::CopyDest, ERHIAccess::CopySrc));

					RUN_TEST(VerifyTextureContents(*TestName, RHICmdList, DestTexture, [&](void* Data, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32, uint32 SliceIndex)
					{
						return CheckTestData(Data, Width * sizeof(uint32), DestDesc.Extent, FIntPoint::ZeroValue, DestDesc.Extent, FIntPoint::ZeroValue, MipIndex);
					}));
				}
			};

			CopyTextureMipsLambda(true);
			CopyTextureMipsLambda(false);

			// Copy region
			{
				const TCHAR* TestName = TEXT("Copy Texture Region");
				const FIntPoint Extent(64, 64);
				const FIntPoint PatternOffsetSource(8, 4);
				const FIntPoint PatternOffsetDest(16, 20);
				const FIntPoint PatternSize(16, 24);
				const int32 ImageIndex = 1;

				const FRHITextureCreateDesc SourceDesc =
					FRHITextureCreateDesc::Create2D(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopySrc);

				FTextureRHIRef SourceTexture = RHICmdList.CreateTexture(SourceDesc);

				uint32 Stride = 0;
				void* Data = RHICmdList.LockTexture2D(SourceTexture, 0, RLM_WriteOnly, Stride, false);
				WriteTestData(Data, Stride, Extent, FIntPoint::ZeroValue, ImageIndex);
				RHICmdList.UnlockTexture2D(SourceTexture, 0, false);

				const FRHITextureCreateDesc DestDesc =
					FRHITextureCreateDesc::Create2D(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopyDest);

				FRHICopyTextureInfo CopyInfo;
				CopyInfo.Size           = FIntVector(PatternSize.X, PatternSize.Y, 1);
				CopyInfo.SourcePosition = FIntVector(PatternOffsetSource.X, PatternOffsetSource.Y, 0);
				CopyInfo.DestPosition   = FIntVector(PatternOffsetDest.X, PatternOffsetDest.Y, 0);

				FTextureRHIRef DestTexture = RHICmdList.CreateTexture(DestDesc);
				RHICmdList.CopyTexture(SourceTexture, DestTexture, CopyInfo);
				RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::CopyDest, ERHIAccess::CopySrc));

				RUN_TEST(VerifyTextureContents(TestName, RHICmdList, DestTexture, [&](void* Data, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 MipIndex, uint32 SliceIndex)
				{
					return CheckTestData(Data, Width * sizeof(uint32), PatternSize, PatternOffsetDest, Extent, PatternOffsetSource, ImageIndex);
				}));
			}

			// Copy source subrect to destination readback buffer
			{
				const TCHAR* TestName = TEXT("Readback Texture Region");
				const FIntPoint Extent(32, 32);
				const FIntPoint PatternOffset(8, 4);
				const FIntPoint PatternSize(16, 24);
				const int32 ImageIndex = 1;

				const FRHITextureCreateDesc SourceDesc =
					FRHITextureCreateDesc::Create2D(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopySrc);

				FTextureRHIRef SourceTexture = RHICmdList.CreateTexture(SourceDesc);

				uint32 Stride = 0;
				void* Data = RHICmdList.LockTexture2D(SourceTexture, 0, RLM_WriteOnly, Stride, false);
				WriteTestData(Data, Stride, Extent, FIntPoint::ZeroValue, ImageIndex);
				RHICmdList.UnlockTexture2D(SourceTexture, 0, false);

				FRHIGPUTextureReadback Readback(TEXT("DestReadbackBuffer"));
				FResolveRect SourceRect = FResolveRect(PatternOffset.X, PatternOffset.Y, PatternOffset.X + PatternSize.X, PatternOffset.Y + PatternSize.Y);

				Readback.EnqueueCopy(RHICmdList, SourceTexture, SourceRect);
				RHICmdList.SubmitAndBlockUntilGPUIdle();

				int32 Pitch;
				void* ReadbackData = Readback.Lock(Pitch);
				bool bReadbackOK = CheckTestData(ReadbackData, Pitch * sizeof(uint32), PatternSize, FIntPoint::ZeroValue, Extent, PatternOffset, ImageIndex);
				if (bReadbackOK)
				{
					UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"%ls\"", TestName);
				}
				else
				{
					UE_LOGF(LogRHIUnitTestCommandlet, Error, "Test failed. \"%ls\"", TestName);
				}
				Readback.Unlock();

				bResult = bResult && bReadbackOK;
			}

			// Copy mip chains
			{
				const TCHAR* TestName = TEXT("Copy Texture Subresources - Mips");
				const FIntPoint Extent(128, 128);
				const int32 NumMips = 4;
				const int32 NumArraySlices = 4;
				const FIntPoint PatternOffsetMip0(8, 16);
				const FIntPoint PatternSizeMip0(64, 96);

				const FRHITextureCreateDesc SourceDesc =
					FRHITextureCreateDesc::Create2DArray(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetNumMips(NumMips) // 128, 64, 32, 16
					.SetArraySize(NumArraySlices)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopySrc);

				FTextureRHIRef SourceTexture = RHICmdList.CreateTexture(SourceDesc);

				int32 ImageIndex = 1;

				/*   SLICES
				  M  1  5  9 13
				  I  2  6 10 14
				  P  3  7 11 15
				  S  4  8 12 16
				*/

				uint32 Strides[NumMips] = {};

				for (int32 ArraySlice = 0; ArraySlice < NumArraySlices; ++ArraySlice)
				{
					for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
					{
						const FIntPoint PatternSize(PatternSizeMip0.X     >> MipIndex, PatternSizeMip0.Y   >> MipIndex);
						const FIntPoint PatternOffset(PatternOffsetMip0.X >> MipIndex, PatternOffsetMip0.Y >> MipIndex);

						void* Data = RHICmdList.LockTexture2DArray(SourceTexture, ArraySlice, MipIndex, RLM_WriteOnly, Strides[MipIndex], false);
						WriteTestData(Data, Strides[MipIndex], PatternSize, PatternOffset, ImageIndex);
						RHICmdList.UnlockTexture2DArray(SourceTexture, ArraySlice, MipIndex, false);

						ImageIndex++;
					}
				}

				const FRHITextureCreateDesc DestDesc =
					FRHITextureCreateDesc::Create2DArray(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetNumMips(NumMips)
					.SetArraySize(NumArraySlices)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopyDest);

				FTextureRHIRef DestTexture = RHICmdList.CreateTexture(DestDesc);

				/*   SLICES
				  M  13  9  5  1
				  I  14 10  6  2
				  P  15 11  7  3
				  S  16 12  8  4
				*/
				for (int32 SourceSliceIndex = 0; SourceSliceIndex < NumArraySlices; ++SourceSliceIndex)
				{
					int32 DestSliceIndex = NumArraySlices - SourceSliceIndex - 1;

					FRHICopyTextureInfo CopyInfo;
					CopyInfo.SourcePosition   = FIntVector(PatternOffsetMip0.X, PatternOffsetMip0.Y, 0);
					CopyInfo.DestPosition     = FIntVector(PatternOffsetMip0.X, PatternOffsetMip0.Y, 0);
					CopyInfo.Size             = FIntVector(PatternSizeMip0.X, PatternSizeMip0.Y, 1);
					CopyInfo.NumMips          = NumMips;
					CopyInfo.SourceSliceIndex = SourceSliceIndex;
					CopyInfo.DestSliceIndex   = DestSliceIndex;

					RHICmdList.CopyTexture(SourceTexture, DestTexture, CopyInfo);
				}

				RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::CopyDest, ERHIAccess::CopySrc));

				RUN_TEST(VerifyTextureContents(TestName, RHICmdList, DestTexture, [&](void* Data, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 MipIndex, uint32 SliceIndex)
				{
					int32 ImageIndex = (NumArraySlices - SliceIndex - 1) * NumMips + MipIndex + 1;

					const FIntPoint PatternSize(PatternSizeMip0.X     >> MipIndex, PatternSizeMip0.Y   >> MipIndex);
					const FIntPoint PatternOffset(PatternOffsetMip0.X >> MipIndex, PatternOffsetMip0.Y >> MipIndex);

					return CheckTestData(Data, Width * sizeof(uint32), PatternSize, PatternOffset, PatternSize, PatternOffset, ImageIndex);
				}));
			}

			// Copy array slices
			{
				const TCHAR* TestName = TEXT("Copy Texture Subresources - Slices");
				const FIntPoint Extent(128, 128);
				const int32 NumMips = 4;
				const int32 NumArraySlices = 4;
				const FIntPoint PatternOffsetMip0(8, 16);
				const FIntPoint PatternSizeMip0(64, 96);

				const FRHITextureCreateDesc SourceDesc =
					FRHITextureCreateDesc::Create2DArray(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetNumMips(NumMips) // 128, 64, 32, 16
					.SetArraySize(NumArraySlices)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopySrc);

				FTextureRHIRef SourceTexture = RHICmdList.CreateTexture(SourceDesc);

				int32 ImageIndex = 1;

				uint32 Strides[NumMips] = {};

				for (int32 ArraySlice = 0; ArraySlice < NumArraySlices; ++ArraySlice)
				{
					for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
					{
						const FIntPoint PatternSize(PatternSizeMip0.X >> MipIndex, PatternSizeMip0.Y >> MipIndex);
						const FIntPoint PatternOffset(PatternOffsetMip0.X >> MipIndex, PatternOffsetMip0.Y >> MipIndex);

						void* Data = RHICmdList.LockTexture2DArray(SourceTexture, ArraySlice, MipIndex, RLM_WriteOnly, Strides[MipIndex], false);
						WriteTestData(Data, Strides[MipIndex], PatternSize, PatternOffset, ImageIndex);
						RHICmdList.UnlockTexture2DArray(SourceTexture, ArraySlice, MipIndex, false);

						ImageIndex++;
					}
				}

				const FRHITextureCreateDesc DestDesc =
					FRHITextureCreateDesc::Create2DArray(TestName)
					.SetExtent(Extent.X, Extent.Y)
					.SetNumMips(NumMips)
					.SetArraySize(NumArraySlices)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopyDest);

				FTextureRHIRef DestTexture = RHICmdList.CreateTexture(DestDesc);

				for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
				{
					FRHICopyTextureInfo CopyInfo;
					CopyInfo.SourcePosition   = FIntVector(PatternOffsetMip0.X >> MipIndex, PatternOffsetMip0.Y >> MipIndex, 0);
					CopyInfo.DestPosition     = FIntVector(PatternOffsetMip0.X >> MipIndex, PatternOffsetMip0.Y >> MipIndex, 0);
					CopyInfo.Size             = FIntVector(PatternSizeMip0.X   >> MipIndex, PatternSizeMip0.Y   >> MipIndex, 1);
					CopyInfo.NumMips          = 1;
					CopyInfo.NumSlices        = NumArraySlices;
					CopyInfo.SourceMipIndex   = MipIndex;
					CopyInfo.DestMipIndex     = MipIndex;

					RHICmdList.CopyTexture(SourceTexture, DestTexture, CopyInfo);
				}

				RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::CopyDest, ERHIAccess::CopySrc));

				RUN_TEST(VerifyTextureContents(TestName, RHICmdList, DestTexture, [&](void* Data, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 MipIndex, uint32 SliceIndex)
				{
					int32 ImageIndex = SliceIndex * NumMips + MipIndex + 1;

					const FIntPoint PatternSize(PatternSizeMip0.X >> MipIndex, PatternSizeMip0.Y >> MipIndex);
					const FIntPoint PatternOffset(PatternOffsetMip0.X >> MipIndex, PatternOffsetMip0.Y >> MipIndex);

					return CheckTestData(Data, Width * sizeof(uint32), PatternSize, PatternOffset, PatternSize, PatternOffset, ImageIndex);
				}));
			}

			// Copy 3D texture
			{
				const TCHAR* TestName = TEXT("Copy 3D Texture");
				
				const FRHITextureCreateDesc SourceDesc =
					FRHITextureCreateDesc::Create3D(TestName)
					.SetExtent(64, 64)
					.SetDepth(64)
					.SetNumMips(6)
					.SetFormat(PF_R32_UINT)
					.SetInitialState(ERHIAccess::CopySrc);

				const FRHITextureCreateDesc DestDesc = FRHITextureCreateDesc(SourceDesc)
					.SetInitialState(ERHIAccess::CopyDest);

				FTextureRHIRef SourceTexture = RHICmdList.CreateTexture(SourceDesc);
				FTextureRHIRef DestTexture = RHICmdList.CreateTexture(DestDesc);

				// Fill the source texture with data
				for (int32 Mip = 0, ImageIndex = 1; Mip < SourceDesc.NumMips; ++Mip, ImageIndex += SourceDesc.Depth)
				{
					FUpdateTextureRegion3D Region(FIntVector::ZeroValue, FIntVector::ZeroValue, FIntVector(
						FMath::Max(SourceDesc.Extent.X >> Mip, 1),
						FMath::Max(SourceDesc.Extent.Y >> Mip, 1),
						FMath::Max(SourceDesc.Depth    >> Mip, 1)
					));

					FUpdateTexture3DData Data = RHICmdList.BeginUpdateTexture3D(SourceTexture, Mip, Region);
					
					const FIntPoint PatternSize(Region.Width, Region.Height);

					for (uint32 Z = 0; Z < Region.Depth; ++Z)
					{
						WriteTestData(Data.Data + Data.DepthPitch * Z, Data.RowPitch, PatternSize, FIntPoint::ZeroValue, ImageIndex + Z);
					}

					RHICmdList.EndUpdateTexture3D(Data);
				}

				FRHICopyTextureInfo CopyInfo;
				CopyInfo.NumMips = SourceDesc.NumMips;
				CopyInfo.NumSlices = SourceDesc.ArraySize;

				RHICmdList.CopyTexture(SourceTexture, DestTexture, CopyInfo);
				RHICmdList.Transition(FRHITransitionInfo(DestTexture, ERHIAccess::CopyDest, ERHIAccess::CopySrc));

				RUN_TEST(VerifyTextureContents(TestName, RHICmdList, DestTexture, [&](void* Data, uint32 MipWidth, uint32 MipHeight, uint32 Width, uint32 Height, uint32 MipIndex, uint32 Depth)
				{
					int32 ImageIndex = SourceDesc.Depth * MipIndex + Depth + 1;

					const FIntPoint PatternSize(MipWidth, MipHeight);

					return CheckTestData(Data, Width * sizeof(uint32), PatternSize, FIntPoint::ZeroValue, PatternSize, FIntPoint::ZeroValue, ImageIndex);
				}));
			}

			// Read surface data
			{
				#define TO_FRHIR10G10B10A2(r, g, b, a) {(uint32)((r * 1023.0f) + 0.5f), (uint32)((g * 1023.0f) + 0.5f), (uint32)((b * 1023.0f) + 0.5f), (uint32)((a * 3.0f) + 0.5f)}
				const TCHAR* TestName = TEXT("Read Surface Data");
				const FIntPoint Extent(16, 16);
				const FIntRect ReadRegion(5, 3, 13, 11);
				const int32 PaletteSize = 30;
				const FRHIR10G10B10A2 Palette[PaletteSize] = {
					TO_FRHIR10G10B10A2(0.2f, 0.4f, 0.6f, 1.0f),
					TO_FRHIR10G10B10A2(0.6f, 0.8f, 1.0f, 1.0f),
					TO_FRHIR10G10B10A2(0.6f, 0.6f, 0.2f, 1.0f),
					TO_FRHIR10G10B10A2(0.4f, 0.4f, 0.6f, 1.0f),
					TO_FRHIR10G10B10A2(0.8f, 0.6f, 0.2f, 1.0f),
					TO_FRHIR10G10B10A2(0.0f, 0.4f, 0.4f, 1.0f),
					TO_FRHIR10G10B10A2(0.2f, 0.6f, 1.0f, 1.0f),
					TO_FRHIR10G10B10A2(0.6f, 0.2f, 0.0f, 1.0f),
					TO_FRHIR10G10B10A2(0.8f, 0.8f, 0.6f, 1.0f),
					TO_FRHIR10G10B10A2(0.4f, 0.4f, 0.4f, 1.0f),
					TO_FRHIR10G10B10A2(1.0f, 0.8f, 0.4f, 1.0f),
					TO_FRHIR10G10B10A2(0.4f, 0.6f, 0.8f, 1.0f),
					TO_FRHIR10G10B10A2(0.4f, 0.2f, 0.4f, 1.0f),
					TO_FRHIR10G10B10A2(0.6f, 0.6f, 0.8f, 1.0f),
					TO_FRHIR10G10B10A2(0.8f, 0.8f, 0.8f, 1.0f),
					TO_FRHIR10G10B10A2(0.4f, 0.6f, 0.6f, 1.0f),
					TO_FRHIR10G10B10A2(0.8f, 0.8f, 0.4f, 1.0f),
					TO_FRHIR10G10B10A2(0.8f, 0.4f, 0.0f, 1.0f),
					TO_FRHIR10G10B10A2(0.6f, 0.6f, 1.0f, 1.0f),
					TO_FRHIR10G10B10A2(0.0f, 0.4f, 0.8f, 1.0f),
					TO_FRHIR10G10B10A2(0.6f, 0.8f, 0.8f, 1.0f),
					TO_FRHIR10G10B10A2(0.6f, 0.6f, 0.6f, 1.0f),
					TO_FRHIR10G10B10A2(1.0f, 0.8f, 0.0f, 1.0f),
					TO_FRHIR10G10B10A2(0.0f, 0.6f, 0.6f, 1.0f),
					TO_FRHIR10G10B10A2(0.6f, 0.8f, 0.2f, 1.0f),
					TO_FRHIR10G10B10A2(1.0f, 0.6f, 0.0f, 1.0f),
					TO_FRHIR10G10B10A2(0.6f, 0.6f, 0.4f, 1.0f),
					TO_FRHIR10G10B10A2(0.4f, 0.8f, 0.8f, 1.0f),
					TO_FRHIR10G10B10A2(0.2f, 0.6f, 0.4f, 1.0f),
					TO_FRHIR10G10B10A2(0.8f, 0.8f, 0.2f, 1.0f),
				};

				const FRHITextureCreateDesc SourceDesc =
					FRHITextureCreateDesc::Create2D(TestName, Extent, PF_A2B10G10R10)
					// Force as non-static
					.AddFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::NoFastClear);

				FTextureRHIRef SourceTexture = RHICmdList.CreateTexture(SourceDesc);

				uint32 Stride = 0;
				void* Data = RHICmdList.LockTexture2D(SourceTexture, 0, RLM_WriteOnly, Stride, false);
				for (int32 Y = 0; Y < Extent.Y; Y++)
				{
					FRHIR10G10B10A2* Ptr = (FRHIR10G10B10A2*)((uint8*)Data + (Y * Stride));
					for (int32 X = 0; X < Extent.X; X++)
					{
						const int32 SurfaceIndex = (Y * Extent.X) + X;
						const int32 PaletteIndex = SurfaceIndex % PaletteSize;
						*Ptr++ = Palette[PaletteIndex];
					}
				}
				RHICmdList.UnlockTexture2D(SourceTexture, 0, false);

				RHICmdList.SubmitAndBlockUntilGPUIdle();

				TArray<FColor> OutData;
				FReadSurfaceDataFlags Flags;
				Flags.SetLinearToGamma(false);
				GDynamicRHI->RHIReadSurfaceData(SourceTexture, ReadRegion, OutData, Flags);

				const auto ValidateRHIReadSurfaceData = [&]()
				{
					FString SampleStr;
					TArray<TTuple<int, int, FRHIR10G10B10A2, FRHIR10G10B10A2>> InvalidSamples;
					for (int32 Y = ReadRegion.Min.Y; Y < ReadRegion.Max.Y; Y++)
					{
						for (int32 X = ReadRegion.Min.X; X < ReadRegion.Max.X; X++)
						{
							const int32 SurfaceIndex = (Y * Extent.X) + X;
							const int32 PaletteIndex = SurfaceIndex % PaletteSize;
							const int32 ReadIndex = ((Y - ReadRegion.Min.Y) * ReadRegion.Width()) + (X - ReadRegion.Min.X);
							const FRHIR10G10B10A2 Expected = Palette[PaletteIndex];
							const FRHIR10G10B10A2 Sample = TO_FRHIR10G10B10A2(((float)OutData[ReadIndex].R / 255.0f), ((float)OutData[ReadIndex].G / 255.0f), ((float)OutData[ReadIndex].B / 255.0f), ((float)OutData[ReadIndex].A / 255.0f));
							if (Sample.R != Expected.R || Sample.G != Expected.G || Sample.B != Expected.B || Sample.A != Expected.A)
							{
								SampleStr.Append(TEXT("X"));
								InvalidSamples.Add(MakeTuple(X, Y, Sample, Expected));
							}
							else
							{
								SampleStr.Append(TEXT("."));
							}
						}
						SampleStr.Append(TEXT("\n"));
					}

					if (InvalidSamples.Num())
					{
						UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test failed. \"%ls\"", TestName);
						UE_LOGF(LogRHIUnitTestCommandlet, Display, "%ls results:\n%ls", TestName, *SampleStr);
						for (const auto& InvalidSample : InvalidSamples)
						{
							UE_LOGF(LogRHIUnitTestCommandlet, Display, "[%2d, %2d]: [%d, %d, %d, %d] != [%d, %d, %d, %d]",
								InvalidSample.Get<0>(), InvalidSample.Get<1>(),
								InvalidSample.Get<2>().R, InvalidSample.Get<2>().G, InvalidSample.Get<2>().B, InvalidSample.Get<2>().A,
								InvalidSample.Get<3>().R, InvalidSample.Get<3>().G, InvalidSample.Get<3>().B, InvalidSample.Get<3>().A);
						}
						return false;
					}
					UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"%ls\"", TestName);
					return true;
				};

				RUN_TEST(ValidateRHIReadSurfaceData());
			}
		}

		return bResult;
		#undef TO_FRHIR10G10B10A2
	}
	
	static bool Test_MultipleLockTexture2D(FRHICommandListImmediate& RHICmdList)
	{
		FString TestName = TEXT("Test_MultipleLockTexture2D");

		// Test the RHI can handle multiple simultaneous locks across texture array slices
		const uint32 ArrayLength = 4;
		const uint32 Size = 8;
		const FIntPoint SliceDim = FIntPoint(Size, Size);
		const EPixelFormat Format = PF_R8G8B8A8;
		FRHITextureCreateDesc DescArray = FRHITextureCreateDesc::Create2DArray(TEXT("Multiple Texture Slice Lock"), SliceDim, ArrayLength, Format).DetermineInititialState();
		
		FTextureRHIRef Texture_SingleLock = RHICmdList.CreateTexture(DescArray);
		FTextureRHIRef Texture_MultipleLock = RHICmdList.CreateTexture(DescArray);
		
		// Create Reasonable Test Data that can be seen in the GPU debugger if required
		const auto WriteSliceTestData = [=](uint8* pDestData, uint32 DestStride, uint32 SliceIdx)
		{
			const uint32 c = 0xffffffff;
			uint32 const Data[] =
			{
				0, 0, 0, c, c, 0, 0, 0,
				0, 0, 0, c, c, 0, 0, 0,
				0, 0, 0, c, c, 0, 0, 0,
				c, c, c, c, c, c, c, c,
				c, c, c, c, c, c, c, c,
				0, 0, 0, c, c, 0, 0, 0,
				0, 0, 0, c, c, 0, 0, 0,
				0, 0, 0, c, c, 0, 0, 0,
			};
			
			uint32 const TintMask[] = {0xff0000ff, 0xff00ff00, 0xffff0000, 0xffffffff};
			
			const uint32 SrcStride = sizeof(uint32) * Size;
			uint8* pSrcData = (uint8*)Data;
		
			for (int32 Y = 0; Y < SliceDim.Y; ++Y)
			{
				uint32* pDestRowData = (uint32*)(pDestData + (Y * DestStride));
				uint32* pSrcRowData = (uint32*)(pSrcData + (Y * SrcStride));

				for (int32 X = 0; X < SliceDim.X;++X)
				{
					pDestRowData[X] = pSrcRowData[X] & TintMask[SliceIdx];
				}
			}
		};
		
		// Lock, write data, then unlock each slice in turn
		{
			for (uint32 SliceIdx = 0; SliceIdx < ArrayLength;++SliceIdx)
			{
				uint32 DestStride = 0;
				uint8* pDestData = (uint8*)RHICmdList.LockTexture2DArray(Texture_SingleLock, SliceIdx, 0, RLM_WriteOnly, DestStride, false);
				WriteSliceTestData(pDestData, DestStride, SliceIdx);
				RHICmdList.UnlockTexture2DArray(Texture_SingleLock, SliceIdx, 0, false);
			}
		}
		
		// Lock all slices, write data, then unlock all slices as individual steps
		{
			uint8* DataPtrs[ArrayLength] = {};
			uint32 SliceStrides[ArrayLength] = {};
			
			for (uint32 SliceIdx = 0; SliceIdx < ArrayLength;++SliceIdx)
			{
				DataPtrs[SliceIdx] = (uint8*)RHICmdList.LockTexture2DArray(Texture_MultipleLock, SliceIdx, 0, RLM_WriteOnly, SliceStrides[SliceIdx], false);
			}
			
			for(uint32 SliceIdx = 0;SliceIdx < ArrayLength;++SliceIdx)
			{
				WriteSliceTestData(DataPtrs[SliceIdx], SliceStrides[SliceIdx], SliceIdx);
			}
			
			for (uint32 SliceIdx = 0; SliceIdx < ArrayLength;++SliceIdx)
			{
				RHICmdList.UnlockTexture2DArray(Texture_MultipleLock, SliceIdx, 0, false);
			}
		}
		
		RHICmdList.Transition({
			FRHITransitionInfo(Texture_SingleLock, DescArray.InitialState, ERHIAccess::CopySrc),
			FRHITransitionInfo(Texture_MultipleLock, DescArray.InitialState, ERHIAccess::CopySrc),
		});

		RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("FRHITextureTests_StagingTexture"), SliceDim.X, SliceDim.Y, Format)
			.SetFlags(ETextureCreateFlags::CPUReadback)
			.SetInitialState(ERHIAccess::CopyDest);

		FTextureRHIRef SingleLockStagingTexture = RHICmdList.CreateTexture(Desc);
		FTextureRHIRef MultiLockStagingTexture = RHICmdList.CreateTexture(Desc);

		FRHICopyTextureInfo CopyInfo = {};
		CopyInfo.Size = FIntVector(SliceDim.X, SliceDim.Y, 1);
		CopyInfo.SourceMipIndex = 0;
		CopyInfo.NumSlices = 1;
		CopyInfo.NumMips = 1;

		// Compare the two textures
		for (uint32 SliceIdx = 0; SliceIdx < ArrayLength;++SliceIdx)
		{
			CopyInfo.SourceSliceIndex = SliceIdx;
			RHICmdList.CopyTexture(Texture_SingleLock, SingleLockStagingTexture, CopyInfo);
			RHICmdList.CopyTexture(Texture_MultipleLock, MultiLockStagingTexture, CopyInfo);

			RHICmdList.Transition({
				FRHITransitionInfo(SingleLockStagingTexture, ERHIAccess::CopyDest, ERHIAccess::CPURead),
				FRHITransitionInfo(MultiLockStagingTexture, ERHIAccess::CopyDest, ERHIAccess::CPURead)
			});

			FGPUFenceRHIRef GPUFence = RHICreateGPUFence(TEXT("ReadbackFence"));
			RHICmdList.WriteGPUFence(GPUFence);

			constexpr bool bFlushResources = true;
			RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

			int32 SingleLockStagingSurfaceWidth, SingleLockStagingSurfaceHeight;
			void* SingleLockData;
			RHICmdList.MapStagingSurface(SingleLockStagingTexture, GPUFence, SingleLockData, SingleLockStagingSurfaceWidth, SingleLockStagingSurfaceHeight);
			check(SingleLockStagingSurfaceWidth >= SliceDim.X && SingleLockStagingSurfaceHeight >= SliceDim.Y);

			int32 MultiLockStagingSurfaceWidth, MultiLockStagingSurfaceHeight;
			void* MultiLockData;
			RHICmdList.MapStagingSurface(MultiLockStagingTexture, GPUFence, MultiLockData, MultiLockStagingSurfaceWidth, MultiLockStagingSurfaceHeight);
			check(MultiLockStagingSurfaceWidth >= SliceDim.X && MultiLockStagingSurfaceHeight >= SliceDim.Y);

			bool bDataMatches = true;
			const uint8* SingleLockCursor = (const uint8*)SingleLockData;
			const uint8* MultiLockCursor = (const uint8*)MultiLockData;
			for (int32 Y = 0; Y < SliceDim.Y; ++Y)
			{
				if (FMemory::Memcmp(SingleLockCursor, MultiLockCursor, SliceDim.X * sizeof(uint32)) != 0)
				{
					bDataMatches = false;
					break;
				}
				SingleLockCursor += SingleLockStagingSurfaceWidth * sizeof(uint32);
				MultiLockCursor += MultiLockStagingSurfaceWidth * sizeof(uint32);
			}

			RHICmdList.UnmapStagingSurface(SingleLockStagingTexture);
			RHICmdList.UnmapStagingSurface(MultiLockStagingTexture);

			if (!bDataMatches)
			{
				UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test failed. \"%ls\"", *TestName);
				return false;
			}

			// Put the staging textures back into CopyDest for the next iteration of the loop.
			RHICmdList.Transition({
				FRHITransitionInfo(SingleLockStagingTexture, ERHIAccess::CPURead, ERHIAccess::CopyDest),
				FRHITransitionInfo(MultiLockStagingTexture, ERHIAccess::CPURead, ERHIAccess::CopyDest)
			});

		}
		
		UE_LOGF(LogRHIUnitTestCommandlet, Display, "Test passed. \"%ls\"", *TestName);
		return true;
	}


};

//UE_ENABLE_OPTIMIZATION
