// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RHITests : ModuleRules
{
	public RHITests(ReadOnlyTargetRules Target) : base(Target)
	{				
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
                "CoreUObject",
                "Engine",
                "RHI",
                "Engine",
                "RenderCore",
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Projects",
			}
		);
	}
}
